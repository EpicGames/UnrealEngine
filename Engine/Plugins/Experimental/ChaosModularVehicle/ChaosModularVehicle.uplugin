{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Modular Vehicle",
	"Description": "Modular Vehicle Integration",
	"Category": "Physics",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosModularVehicle",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "ChaosModularVehicleEngine",
			"Type": "Runtime",
			"LoadingPhase": "PreDefault"
		},
		{
			"Name": "ChaosModularVehicleEditor",
			"Type": "UncookedOnly",
			"LoadingPhase": "PreDefault"
		}

	],
	"Plugins": [
		{
			"Name": "EnhancedInput",
			"Enabled": true
		}
	]


}