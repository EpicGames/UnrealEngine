// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ChaosModularVehiclePlugin.h"
#include "Features/IModularFeatures.h"
#include "Modules/ModuleManager.h"

//////////////////////////////////////////////////////////////////////////
// FChaosModularVehiclePlugin
class FChaosModularVehiclePlugin : public IChaosModularVehiclePlugin
{
  public:
    virtual void StartupModule() override
    {
  //      check(GConfig);
		//OnPhysSceneInitHandle = FPhysicsDelegates::OnPhysSceneInit.AddRaw(this, &FChaosModularVehiclePlugin::PhysSceneInit);
		//OnPhysSceneTermHandle = FPhysicsDelegates::OnPhysSceneTerm.AddRaw(this, &FChaosModularVehiclePlugin::PhysSceneTerm);
    }

    virtual void ShutdownModule() override
    {
		//FPhysicsDelegates::OnPhysSceneInit.Remove(OnPhysSceneInitHandle);
		//FPhysicsDelegates::OnPhysSceneTerm.Remove(OnPhysSceneTermHandle);
	}

//	void PhysSceneInit(FPhysScene* PhysScene)
//	{
//#if WITH_CHAOS
//		new FChaosSimModuleManager(PhysScene);
//#endif
//	}
//
//	void PhysSceneTerm(FPhysScene* PhysScene)
//	{
//#if WITH_CHAOS
//		FChaosSimModuleManager* VehicleManager = FChaosSimModuleManager::GetManagerFromScene(PhysScene);
//		if (VehicleManager != nullptr)
//		{
//			VehicleManager->DetachFromPhysScene(PhysScene);
//			delete VehicleManager;
//			VehicleManager = nullptr;
//		}
//#endif // WITH_CHAOS
//	}
//
//	FDelegateHandle OnPhysSceneInitHandle;
//	FDelegateHandle OnPhysSceneTermHandle;

};

IMPLEMENT_MODULE(FChaosModularVehiclePlugin, ChaosModularVehicle);
