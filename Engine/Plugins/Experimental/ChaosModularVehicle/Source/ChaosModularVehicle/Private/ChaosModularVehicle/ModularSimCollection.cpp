// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ModularSimulationCollection.cpp: FModularSimCollection methods.
=============================================================================*/

#include "ChaosModularVehicle/ModularSimCollection.h"

DEFINE_LOG_CATEGORY_STATIC(FModularSimCollectionLogging, Log, All);


// Attributes
const FName FModularSimCollection::SimModuleIndexAttribute("SimModuleIndex");

FModularSimCollection::FModularSimCollection()
	: FGeometryCollection()
{
	Construct();
}


void FModularSimCollection::Construct()
{
}


FModularSimCollection* FModularSimCollection::NewModularSimulationCollection()
{

	FModularSimCollection* Collection = new FModularSimCollection();
	FModularSimCollection::Init(Collection);
	return Collection;
}
FModularSimCollection* FModularSimCollection::NewModularSimulationCollection(const FTransformCollection& Base)
{
	FModularSimCollection* Collection = new FModularSimCollection();
	Collection->CopyMatchingAttributesFrom(Base);
	return Collection;
}

void FModularSimCollection::Init(FModularSimCollection* Collection)
{
	if (Collection)
	{
	}
}

void FModularSimCollection::GenerateSimTree()
{

}


