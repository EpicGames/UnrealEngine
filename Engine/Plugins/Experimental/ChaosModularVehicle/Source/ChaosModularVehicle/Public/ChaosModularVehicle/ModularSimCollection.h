// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GeometryCollection/GeometryCollection.h"

#define UE_API CHAOSMODULARVEHICLE_API

namespace Chaos
{
	class FChaosArchive;
}

/**
* FModularSimCollection (FTransformCollection)
*/
//class CHAOSMODULARVEHICLE_API FModularSimCollection : public FTransformCollection
class FModularSimCollection : public FGeometryCollection
{
public:
	//typedef FTransformCollection Super;
	typedef FGeometryCollection Super;

	UE_API FModularSimCollection();
	FModularSimCollection(FModularSimCollection&) = delete;
	FModularSimCollection& operator=(const FModularSimCollection&) = delete;
	FModularSimCollection(FModularSimCollection&&) = default;
	FModularSimCollection& operator=(FModularSimCollection&&) = default;

	/**
	 * Create a GeometryCollection from Vertex and Indices arrays
	 */
	static UE_API FModularSimCollection* NewModularSimulationCollection(const FTransformCollection& Base);
	static UE_API FModularSimCollection* NewModularSimulationCollection();
	static UE_API void Init(FModularSimCollection* Collection);

	/*
	* Index of simulation module associated with each transform node
	*   FManagedArray<int32> SimModuleIndex = this->FindAttribute<int32>("SimModuleIndex",FGeometryCollection::TransformGroup);
	*/
	static UE_API const FName SimModuleIndexAttribute;
	TManagedArray<int32> SimModuleIndex;

	// TODO: need to do this conversion somewhere - putting here for now
	UE_API void GenerateSimTree();

protected:
	UE_API void Construct();

};

inline Chaos::FChaosArchive& operator<<(Chaos::FChaosArchive& Ar, FModularSimCollection& Value)
{
	Value.Serialize(Ar);
	return Ar;
}

#undef UE_API
