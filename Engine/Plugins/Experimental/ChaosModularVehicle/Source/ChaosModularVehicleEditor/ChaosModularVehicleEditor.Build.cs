// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosModularVehicleEditor : ModuleRules
	{
		public ChaosModularVehicleEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			SetupModulePhysicsSupport(Target);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Chaos",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"PropertyEditor",
					"AnimGraphRuntime",
					"AnimGraph",
					"BlueprintGraph",
					"ToolMenus",
					"AnimGraphRuntime",
					"ChaosModularVehicleEngine",
					"NetCore"
				}
			);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
			SetupIrisSupport(Target);
		}
	}
}
