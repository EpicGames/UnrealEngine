// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/AnimGraphNode_ModularVehicleController.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "Kismet2/CompilerResultsLog.h"

/////////////////////////////////////////////////////
// UAnimGraphNode_ModularVehicleController

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimGraphNode_ModularVehicleController)

#define LOCTEXT_NAMESPACE "ModularVehicle"

UAnimGraphNode_ModularVehicleController::UAnimGraphNode_ModularVehicleController(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

FText UAnimGraphNode_ModularVehicleController::GetControllerDescription() const
{
	return LOCTEXT("AnimGraphNode_ModularVehicleController", "Controller for ModularVehicle");
}

FText UAnimGraphNode_ModularVehicleController::GetTooltipText() const
{
	return LOCTEXT("AnimGraphNode_ModularVehicleController_Tooltip", "This alters the transform based on set up in Modular Vehicle. This only works when the owner is a modular vehicle.");
}

FText UAnimGraphNode_ModularVehicleController::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	FText NodeTitle;
	if (TitleType == ENodeTitleType::ListView || TitleType == ENodeTitleType::MenuTitle)
	{
		NodeTitle = GetControllerDescription();
	}
	else
	{
		// we don't have any run-time information, so it's limited to print  
		// anymore than what it is it would be nice to print more data such as 
		// name of bones for wheels, but it's not available in Persona
		NodeTitle = FText(LOCTEXT("AnimGraphNode_ModularVehicleController_Title", "Modular Vehicle Controller"));
	}	
	return NodeTitle;
}

void UAnimGraphNode_ModularVehicleController::ValidateAnimNodePostCompile(class FCompilerResultsLog& MessageLog, class UAnimBlueprintGeneratedClass* CompiledClass, int32 CompiledNodeIndex)
{
	// we only support vehicle anim instance
	if ( CompiledClass->IsChildOf(UModularVehicleAnimationInstance::StaticClass())  == false )
	{
		MessageLog.Error(TEXT("@@ is only allowed in ModularVehicleAnimInstance. If this is for vehicle, please change parent to be ModularVehicleAnimInstance (Reparent Class)."), this);
	}
}

bool UAnimGraphNode_ModularVehicleController::IsCompatibleWithGraph(const UEdGraph* TargetGraph) const
{
	UBlueprint* Blueprint = FBlueprintEditorUtils::FindBlueprintForGraph(TargetGraph);
	return (Blueprint != nullptr) && Blueprint->ParentClass->IsChildOf<UModularVehicleAnimationInstance>() && Super::IsCompatibleWithGraph(TargetGraph);
}

#undef LOCTEXT_NAMESPACE
