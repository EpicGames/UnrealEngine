// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ChaosModularVehicleEditorPlugin.h"

#include "Modules/ModuleManager.h"
#include "ChaosModularVehicle/ChaosSimModuleManager.h"


class FChaosModularVehicleEditorPlugin : public IChaosModularVehicleEditorPlugin
{
  public:
	/** IModuleInterface implementation */

	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}

};

IMPLEMENT_MODULE(FChaosModularVehicleEditorPlugin, ChaosModularVehicleEditor)
