// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AnimGraphNode_SkeletalControlBase.h"
#include "ChaosModularVehicle/AnimNode_ModularVehicleController.h"
#include "AnimGraphNode_ModularVehicleController.generated.h"

class UEdGraph;

UCLASS(MinimalAPI, meta=(Keywords = "Modify Modular Vehicle"))
class UAnimGraphNode_ModularVehicleController : public UAnimGraphNode_SkeletalControlBase
{
	GENERATED_UCLASS_BODY()

	UPROPERTY(EditAnywhere, Category=Settings)
	FAnimNode_ModularVehicleController Node;

public:
	// UEdGraphNode interface
	virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	virtual FText GetTooltipText() const override;
	// validate if this is within VehicleAnimInstance
	virtual void ValidateAnimNodePostCompile(class FCompilerResultsLog& MessageLog, class UAnimBlueprintGeneratedClass* CompiledClass, int32 CompiledNodeIndex) override;
	virtual bool IsCompatibleWithGraph(const UEdGraph* TargetGraph) const override;
	// End of UEdGraphNode interface

protected:
	// UAnimGraphNode_SkeletalControlBase interface
	virtual FText GetControllerDescription() const override;
	virtual const FAnimNode_SkeletalControlBase* GetNode() const override { return &Node; }
	// End of UAnimGraphNode_SkeletalControlBase interface
};
