// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class ChaosModularVehicleEngine : ModuleRules
	{
		public ChaosModularVehicleEngine(ReadOnlyTargetRules Target) : base(Target)
		{
			SetupModulePhysicsSupport(Target);

			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Chaos",
					"Engine",
					"RenderCore",
					"RHI",
					"EnhancedInput",
					"AnimGraphRuntime",
					"Renderer",
					"ChaosVehiclesCore",
					"ChaosModularVehicle",
					"NetCore",
					"GeometryCollectionEngine",
					"ChaosSolverEngine"
				}
			);

			PrivateDefinitions.Add("CHAOS_INCLUDE_LEVEL_1=1");
			SetupIrisSupport(Target);
		}
	}
}
