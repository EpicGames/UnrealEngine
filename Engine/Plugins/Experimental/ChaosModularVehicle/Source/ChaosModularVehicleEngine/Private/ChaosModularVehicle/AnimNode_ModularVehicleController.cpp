// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/AnimNode_ModularVehicleController.h"
#include "Animation/AnimTrace.h"
#include "AnimationRuntime.h"
#include "Animation/AnimStats.h"
#include "SimModule/SimulationModuleBase.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AnimNode_ModularVehicleController)


FAnimNode_ModularVehicleController::FAnimNode_ModularVehicleController()
{
	AnimInstanceProxy = nullptr;
}

void FAnimNode_ModularVehicleController::GatherDebugData(FNodeDebugData& DebugData)
{
	FString DebugLine = DebugData.GetNodeName(this);

	DebugLine += "(";
	AddDebugNodeData(DebugLine);
	DebugLine += ")";

	DebugData.AddDebugItem(DebugLine);

	const TArray<FModuleAnimationData>& AnimData = AnimInstanceProxy->GetModuleAnimData();
	for (const FModuleLookupData& Module : Modules)
	{
		if (Module.BoneReference.BoneIndex != INDEX_NONE)
		{
			DebugLine = FString::Printf(TEXT(" [Module Index : %d] Bone: %s , Rotation Offset : %s, Location Offset : %s"),
				Module.ModuleIndex, *Module.BoneReference.BoneName.ToString(), *AnimData[Module.ModuleIndex].RotOffset.ToString(), *AnimData[Module.ModuleIndex].LocOffset.ToString());
		}
		else
		{
			DebugLine = FString::Printf(TEXT(" [Module Index : %d] Bone: %s (invalid bone)"),
				Module.ModuleIndex, *Module.BoneReference.BoneName.ToString());
		}

		DebugData.AddDebugItem(DebugLine);
	}

	ComponentPose.GatherDebugData(DebugData);
}

void FAnimNode_ModularVehicleController::EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms)
{
	check(OutBoneTransforms.Num() == 0);

	ANIM_MT_SCOPE_CYCLE_COUNTER_VERBOSE(ModularVehicleController, !IsInGameThread());

	const TArray<FModuleAnimationData>& ModuleAnimData = AnimInstanceProxy->GetModuleAnimData();

	const FBoneContainer& BoneContainer = Output.Pose.GetPose().GetBoneContainer();
	for(const FModuleLookupData& Module : Modules)
	{
		if (Module.BoneReference.IsValidToEvaluate(BoneContainer))
		{
			if (Module.ModuleIndex < ModuleAnimData.Num())
			{
				FCompactPoseBoneIndex ModuleSimBoneIndex = Module.BoneReference.GetCompactPoseIndex(BoneContainer);

				// the way we apply transform is same as FMatrix or FTransform
				// we apply scale first, and rotation, and translation
				// if you'd like to translate first, you'll need two nodes that first node does translate and second nodes to rotate. 
				FTransform NewBoneTM = Output.Pose.GetComponentSpaceTransform(ModuleSimBoneIndex);

				FAnimationRuntime::ConvertCSTransformToBoneSpace(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, NewBoneTM, ModuleSimBoneIndex, BCS_ComponentSpace);

				if (ModuleAnimData[Module.ModuleIndex].Flags & Chaos::EAnimationFlags::AnimateRotation)
				{
					// Apply rotation offset
					const FQuat BoneQuat(ModuleAnimData[Module.ModuleIndex].RotOffset);
					NewBoneTM.SetRotation(BoneQuat * NewBoneTM.GetRotation());
				}

				if (ModuleAnimData[Module.ModuleIndex].Flags & Chaos::EAnimationFlags::AnimatePosition)
				{
					// Apply loc offset
					NewBoneTM.AddToTranslation(ModuleAnimData[Module.ModuleIndex].LocOffset);
				}

				// Convert back to Component Space.
				FAnimationRuntime::ConvertBoneSpaceTransformToCS(Output.AnimInstanceProxy->GetComponentTransform(), Output.Pose, NewBoneTM, ModuleSimBoneIndex, BCS_ComponentSpace);

				// add back to it
				OutBoneTransforms.Add(FBoneTransform(ModuleSimBoneIndex, NewBoneTM));
			}
		}
	}

#if ANIM_TRACE_ENABLED
	for (const FModuleLookupData& Module : Modules)
	{
		if ((Module.BoneReference.BoneIndex != INDEX_NONE) && (Module.ModuleIndex < ModuleAnimData.Num()))
		{
			TRACE_ANIM_NODE_VALUE(Output, *FString::Printf(TEXT("Module %d Name"), Module.ModuleIndex), *Module.BoneReference.BoneName.ToString());
			TRACE_ANIM_NODE_VALUE(Output, *FString::Printf(TEXT("Module %d Rotation Offset"), Module.ModuleIndex), ModuleAnimData[Module.ModuleIndex].RotOffset);
			TRACE_ANIM_NODE_VALUE(Output, *FString::Printf(TEXT("Module %d Location Offset"), Module.ModuleIndex), ModuleAnimData[Module.ModuleIndex].LocOffset);
		}
		else
		{
			TRACE_ANIM_NODE_VALUE(Output, *FString::Printf(TEXT("Module %d Name"), Module.ModuleIndex), *FString::Printf(TEXT("%s (invalid)"), *Module.BoneReference.BoneName.ToString()));
		}
	}
#endif
}

bool FAnimNode_ModularVehicleController::IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) 
{
	if (AnimInstanceProxy)
	{
		// Note sure the best way to initilaize the animation since vehicle construction happens quite late on BeginPlay
		const TArray<FModuleAnimationData>& ModuleAnimData = AnimInstanceProxy->GetModuleAnimData();
		if (ModuleAnimData.Num() != Modules.Num())
		{
			InitializeBoneReferences(RequiredBones);
		}
	}

	// if both bones are valid
	for (const FModuleLookupData& Module : Modules)
	{
		// if one of them is valid
		if (Module.BoneReference.IsValidToEvaluate(RequiredBones) == true)
		{
			return true;
		}
	}

	return false;
}

void FAnimNode_ModularVehicleController::InitializeBoneReferences(const FBoneContainer& RequiredBones) 
{
	const TArray<FModuleAnimationData>& ModuleAnimData = AnimInstanceProxy->GetModuleAnimData();
	const int32 NumModules = ModuleAnimData.Num();
	Modules.Empty(NumModules);

	for (int32 ModuleIndex = 0; ModuleIndex < NumModules; ++ModuleIndex)
	{
		FModuleLookupData* Module = new(Modules)FModuleLookupData();
		Module->ModuleIndex = ModuleIndex;
		Module->BoneReference.BoneName = ModuleAnimData[ModuleIndex].BoneName;
		Module->BoneReference.Initialize(RequiredBones);
	}

	// sort by bone indices
	Modules.Sort([](const FModuleLookupData& L, const FModuleLookupData& R) { return L.BoneReference.BoneIndex < R.BoneReference.BoneIndex; });
}

void FAnimNode_ModularVehicleController::Initialize_AnyThread(const FAnimationInitializeContext& Context)
{
	FAnimNode_SkeletalControlBase::Initialize_AnyThread(Context);

	AnimInstanceProxy = (FModularVehicleAnimationInstanceProxy*)Context.AnimInstanceProxy;
}


