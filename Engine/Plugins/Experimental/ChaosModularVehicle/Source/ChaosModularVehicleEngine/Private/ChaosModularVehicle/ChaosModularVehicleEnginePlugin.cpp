// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ChaosModularVehicleEnginePlugin.h"

#include "Modules/ModuleManager.h"
#include "ChaosModularVehicle/ChaosSimModuleManager.h"
#include "PhysicsPublic.h"

class FChaosModularVehicleEnginePlugin : public IChaosModularVehicleEnginePlugin
{
  public:
	/** IModuleInterface implementation */

	virtual void StartupModule() override
	{
		check(GConfig);
		OnPhysSceneInitHandle = FPhysicsDelegates::OnPhysSceneInit.AddRaw(this, &FChaosModularVehicleEnginePlugin::PhysSceneInit);
		OnPhysSceneTermHandle = FPhysicsDelegates::OnPhysSceneTerm.AddRaw(this, &FChaosModularVehicleEnginePlugin::PhysSceneTerm);
	}

	virtual void ShutdownModule() override
	{
		FPhysicsDelegates::OnPhysSceneInit.Remove(OnPhysSceneInitHandle);
		FPhysicsDelegates::OnPhysSceneTerm.Remove(OnPhysSceneTermHandle);
	}

	void PhysSceneInit(FPhysScene* PhysScene)
	{
		new FChaosSimModuleManager(PhysScene);
	}

	void PhysSceneTerm(FPhysScene* PhysScene)
	{
		FChaosSimModuleManager* VehicleManager = FChaosSimModuleManager::GetManagerFromScene(PhysScene);
		if (VehicleManager != nullptr)
		{
			VehicleManager->DetachFromPhysScene(PhysScene);
			delete VehicleManager;
			VehicleManager = nullptr;
		}
	}

	FDelegateHandle OnPhysSceneInitHandle;
	FDelegateHandle OnPhysSceneTermHandle;

};

IMPLEMENT_MODULE(FChaosModularVehicleEnginePlugin, ChaosModularVehicleEngine)
