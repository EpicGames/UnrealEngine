//// Copyright Epic Games, Inc. All Rights Reserved.
//
#include "ChaosModularVehicle/ChaosSimModuleManager.h"
#include "ChaosModularVehicle/ModuleInputTokenStore.h"
#include "ChaosModularVehicle/ModularVehicleBaseComponent.h"
#include "PBDRigidsSolver.h"
#include "GameFramework/HUD.h" // for ShowDebugInfo
#include "Physics/Experimental/PhysScene_Chaos.h"
TMap<FPhysScene*, FChaosSimModuleManager*> FChaosSimModuleManager::SceneToModuleManagerMap;

FDelegateHandle FChaosSimModuleManager::OnPostWorldInitializationHandle;
FDelegateHandle FChaosSimModuleManager::OnWorldCleanupHandle;

extern FSimModuleDebugParams GSimModuleDebugParams;

bool FChaosSimModuleManager::GInitialized = false;


FChaosSimModuleManager::FChaosSimModuleManager(FPhysScene* PhysScene)
	: Scene(*PhysScene)
	, AsyncCallback(nullptr)
	, Timestamp(0)

{
	check(PhysScene);
	
	if (!GInitialized)
	{
		GInitialized = true;
		// PhysScene->GetOwningWorld() is always null here, the world is being setup too late to be of use
		// therefore setup these global world delegates that will callback when everything is setup so registering
		// the physics solver Async Callback will succeed
		OnPostWorldInitializationHandle = FWorldDelegates::OnPostWorldInitialization.AddStatic(&FChaosSimModuleManager::OnPostWorldInitialization);
		OnWorldCleanupHandle = FWorldDelegates::OnWorldCleanup.AddStatic(&FChaosSimModuleManager::OnWorldCleanup);

		if (!IsRunningDedicatedServer())
		{
			AHUD::OnShowDebugInfo.AddStatic(&FChaosSimModuleManager::OnShowDebugInfo);
		}
	}

	ensure(FChaosSimModuleManager::SceneToModuleManagerMap.Find(PhysScene) == nullptr);	// double registration with same scene, will cause a leak

	// Add to Scene-To-Manager map
	FChaosSimModuleManager::SceneToModuleManagerMap.Add(PhysScene, this);
}

FChaosSimModuleManager::~FChaosSimModuleManager()
{
	while (CUVehicles.Num() > 0)
	{
		RemoveVehicle(CUVehicles.Last());
	}
}

void FChaosSimModuleManager::OnPostWorldInitialization(UWorld* InWorld, const UWorld::InitializationValues)
{
	FChaosSimModuleManager* Manager = FChaosSimModuleManager::GetManagerFromScene(InWorld->GetPhysicsScene());
	if (Manager)
	{
		Manager->RegisterCallbacks(InWorld);
	}
}

void FChaosSimModuleManager::OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources)
{
	FChaosSimModuleManager* Manager = FChaosSimModuleManager::GetManagerFromScene(InWorld->GetPhysicsScene());
	if (Manager)
	{
		Manager->UnregisterCallbacks();
	}
}

void FChaosSimModuleManager::OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos)
{
	static const FName NAME_ModularVehicle("ModularVehicle");
	if (Canvas && HUD->ShouldDisplayDebug(NAME_ModularVehicle))
	{
		if (FChaosSimModuleManager* Manager = FChaosSimModuleManager::GetManagerFromScene(HUD->GetWorld()->GetPhysicsScene()))
		{
			int ShowVehicleIndex = 0;
			if (!Manager->CUVehicles.IsEmpty())
			{
				TStrongObjectPtr<UModularVehicleBaseComponent> StrongPtr = Manager->CUVehicles[ShowVehicleIndex].Pin();
				if (StrongPtr.IsValid())
				{
					StrongPtr->ShowDebugInfo(HUD, Canvas, DisplayInfo, YL, YPos);
				}
			}
		}
	}
}

void FChaosSimModuleManager::DetachFromPhysScene(FPhysScene* PhysScene)
{
	if (AsyncCallback)
	{
		UnregisterCallbacks();
	}

	FChaosSimModuleManager::SceneToModuleManagerMap.Remove(PhysScene);
}

void FChaosSimModuleManager::OnNetDriverCreated(UWorld* InWorld, UNetDriver* InNetDriver)
{
	if (InNetDriver)
	{
		if (UE::Net::FNetTokenStore* TokenStore = InNetDriver->GetNetTokenStore())
		{
			RegisterNetTokenDataStores(InNetDriver);
		}
		else
		{
			InNetDriver->OnNetTokenStoreReady().AddRaw(this,&FChaosSimModuleManager::RegisterNetTokenDataStores);
		}
	}
}

void FChaosSimModuleManager::RegisterNetTokenDataStores(UNetDriver* InNetDriver)
{
	if (InNetDriver)
	{
		if (UE::Net::FNetTokenStore* TokenStore = InNetDriver->GetNetTokenStore())
		{
			typedef UE::Net::TStructNetTokenDataStore<FModuleInputNetTokenData> FModuleInputNetTokenStore;
			typedef UE::Net::TStructNetTokenDataStore<FNetworkModularVehicleStateNetTokenData> FNetworkModularVehicleStateNetTokenStore;
			if (!TokenStore->GetDataStore<FModuleInputNetTokenStore>())
			{
				TokenStore->CreateAndRegisterDataStore<FModuleInputNetTokenStore>();
			}
			if (!TokenStore->GetDataStore<FNetworkModularVehicleStateNetTokenStore>())
			{
				TokenStore->CreateAndRegisterDataStore<FNetworkModularVehicleStateNetTokenStore>();
			}
		}
	}
}

void FChaosSimModuleManager::RegisterCallbacks(UWorld* InWorld)
{
	OnNetDriverCreatedHandle = FWorldDelegates::OnNetDriverCreated.AddRaw(this, &FChaosSimModuleManager::OnNetDriverCreated);
	OnPhysScenePreTickHandle = Scene.OnPhysScenePreTick.AddRaw(this, &FChaosSimModuleManager::Update);
	OnPhysScenePostTickHandle = Scene.OnPhysScenePostTick.AddRaw(this, &FChaosSimModuleManager::PostUpdate);

	// Set up our async object manager to handle async ticking and marshaling
	check(AsyncCallback == nullptr);
	AsyncCallback = Scene.GetSolver()->CreateAndRegisterSimCallbackObject_External<FChaosSimModuleManagerAsyncCallback>();

	if (FNetworkPhysicsCallback* SolverCallback = static_cast<FNetworkPhysicsCallback*>(Scene.GetSolver()->GetRewindCallback()))
	{
		SolverCallback->InjectInputsExternal.AddRaw(this, &FChaosSimModuleManager::InjectInputs_External);
	}
}

void FChaosSimModuleManager::UnregisterCallbacks()
{
	Scene.OnPhysScenePreTick.Remove(OnPhysScenePreTickHandle);
	Scene.OnPhysScenePostTick.Remove(OnPhysScenePostTickHandle);
	FWorldDelegates::OnNetDriverCreated.Remove(OnNetDriverCreatedHandle);
	
	if (AsyncCallback)
	{
		Scene.GetSolver()->UnregisterAndFreeSimCallbackObject_External(AsyncCallback);
		AsyncCallback = nullptr;
	}

}

FChaosSimModuleManager* FChaosSimModuleManager::GetManagerFromScene(FPhysScene* PhysScene)
{
	FChaosSimModuleManager* Manager = nullptr;
	FChaosSimModuleManager** ManagerPtr = SceneToModuleManagerMap.Find(PhysScene);
	if (ManagerPtr != nullptr)
	{
		Manager = *ManagerPtr;
	}
	return Manager;
}

void FChaosSimModuleManager::AddVehicle(TWeakObjectPtr<UModularVehicleBaseComponent> Vehicle)
{
	check(Vehicle != NULL);
	check(AsyncCallback);
	CUVehicles.Add(Vehicle);
}

void FChaosSimModuleManager::RemoveVehicle(TWeakObjectPtr<UModularVehicleBaseComponent> Vehicle)
{
	if (Vehicle != nullptr)
	{
		CUVehicles.Remove(Vehicle);
	}
}

void FChaosSimModuleManager::ScenePreTick(FPhysScene* PhysScene, float DeltaTime)
{
	for (TWeakObjectPtr<UModularVehicleBaseComponent>& Vehicle : CUVehicles)
	{
		TStrongObjectPtr<UModularVehicleBaseComponent> StrongPtr = Vehicle.Pin();
		if (StrongPtr.IsValid())
		{
			StrongPtr->PreTickGT(DeltaTime);
		}
	}
}

void FChaosSimModuleManager::Update(FPhysScene* PhysScene, float DeltaTime)
{
	UWorld* World = Scene.GetOwningWorld();

	SubStepCount = 0;

	ScenePreTick(PhysScene, DeltaTime);

	if (World)
	{
		FChaosSimModuleManagerAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External();

		for (TWeakObjectPtr<UModularVehicleBaseComponent>& Vehicle : CUVehicles)
		{
			TStrongObjectPtr<UModularVehicleBaseComponent> StrongPtr = Vehicle.Pin();
			if (StrongPtr.IsValid())
			{
				StrongPtr->Update(DeltaTime);
				StrongPtr->FinalizeSimCallbackData(*AsyncInput);
			}
		}
	}
}


void FChaosSimModuleManager::PostUpdate(FChaosScene* PhysScene)
{
	ParallelUpdateVehicles();

	for (TWeakObjectPtr<UModularVehicleBaseComponent>& Vehicle : CUVehicles)
	{
		TStrongObjectPtr<UModularVehicleBaseComponent> StrongPtr = Vehicle.Pin();
		if (StrongPtr.IsValid())
		{
			StrongPtr->PostUpdate();
		}
	}
}

void FChaosSimModuleManager::InjectInputs_External(int32 PhysicsStep, int32 NumSteps)
{
	UWorld* World = Scene.GetOwningWorld();
	if (IsValid(World) == false)
	{
		return;
	}
	FChaosSimModuleManagerAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External();
	check(AsyncInput);

	AsyncInput->Reset();	// only want latest frame's data
	AsyncInput->VehicleInputs.Reserve(CUVehicles.Num());
	AsyncInput->World = World;
	AsyncInput->Timestamp = Timestamp;
	++Timestamp;

	for (TWeakObjectPtr<UModularVehicleBaseComponent>& Vehicle : CUVehicles)
	{
		TStrongObjectPtr<UModularVehicleBaseComponent> StrongPtr = Vehicle.Pin();
		if (StrongPtr.IsValid())
		{
			TUniquePtr<FModularVehicleAsyncInput> CurInput = MakeUnique<FModularVehicleAsyncInput>();
			AsyncInput->VehicleInputs.Add(MoveTemp(CurInput));
			StrongPtr->ProduceInput(PhysicsStep, NumSteps, AsyncInput->VehicleInputs.Last().Get());
		}
	}
}

void FSimModuleOutputRecord::ConsumeOutput(Chaos::TSimCallbackOutputHandle<FChaosSimModuleManagerAsyncOutput>&& Output)
{
	if (!CachedOutput_0)
	{
		CachedOutput_0 = MoveTemp(Output);
	}
	else if (!CachedOutput_1)
	{
		CachedOutput_1 = MoveTemp(Output);
	}
	else
	{
		(bPreviousOutputIs0 ? CachedOutput_0 : CachedOutput_1) = MoveTemp(Output);
		bPreviousOutputIs0 = !bPreviousOutputIs0;
	}
}

FChaosSimModuleManagerAsyncOutput* FSimModuleOutputRecord::GetPreviousOutput()
{
	if (CachedOutput_0 && bPreviousOutputIs0)
	{
		return CachedOutput_0.Get();
	}
	if (CachedOutput_1 && !bPreviousOutputIs0)
	{
		return CachedOutput_1.Get();
	}
	return nullptr;
}

FChaosSimModuleManagerAsyncOutput* FSimModuleOutputRecord::GetNextOutput()
{
	if (CachedOutput_1 && bPreviousOutputIs0)
	{
		return CachedOutput_1.Get();
	}
	if (CachedOutput_0 && !bPreviousOutputIs0)
	{
		return CachedOutput_0.Get();
	}
	return nullptr;
}

const FChaosSimModuleManagerAsyncOutput* FSimModuleOutputRecord::GetPreviousOutput() const
{
	if (CachedOutput_0 && bPreviousOutputIs0)
	{
		return CachedOutput_0.Get();
	}
	if (CachedOutput_1 && !bPreviousOutputIs0)
	{
		return CachedOutput_1.Get();
	}
	return nullptr;
}

const FChaosSimModuleManagerAsyncOutput* FSimModuleOutputRecord::GetNextOutput() const
{
	if (CachedOutput_1 && bPreviousOutputIs0)
	{
		return CachedOutput_1.Get();
	}
	if (CachedOutput_0 && !bPreviousOutputIs0)
	{
		return CachedOutput_0.Get();
	}
	return nullptr;
}

double FSimModuleOutputRecord::GetLatestOutputStartTime() const
{
	double LatestOutputStartTime = -DBL_MAX;
	if (CachedOutput_0 && CachedOutput_1)
	{
		LatestOutputStartTime = (bPreviousOutputIs0 ? CachedOutput_1 : CachedOutput_0)->InternalTime;
	}
	else if (CachedOutput_0)
	{
		LatestOutputStartTime = CachedOutput_0->InternalTime;
	}
	return LatestOutputStartTime;
}

double FSimModuleOutputRecord::GetInterpolationFactor(double AtInternalTime) const
{
	const FChaosSimModuleManagerAsyncOutput* PreviousOutput = GetPreviousOutput();
	const FChaosSimModuleManagerAsyncOutput* NextOutput = GetNextOutput();
	double Alpha = 0.f;
	if (NextOutput)
	{
		const double Denom = NextOutput->InternalTime - PreviousOutput->InternalTime;
		if (Denom > SMALL_NUMBER)
		{
			Alpha = FMath::Clamp((AtInternalTime - PreviousOutput->InternalTime) / Denom, 0.0f, 1.0f);
		}
	}
	return Alpha;
}

void FSimModuleOutputRecord::Clear()
{
	CachedOutput_0 = Chaos::TSimCallbackOutputHandle<FChaosSimModuleManagerAsyncOutput>();
	CachedOutput_1 = Chaos::TSimCallbackOutputHandle<FChaosSimModuleManagerAsyncOutput>();
	bPreviousOutputIs0 = true;
}

void FChaosSimModuleManager::ParallelUpdateVehicles()
{
	// Friendly reminder: Results time is the time at the END of an async step
	const double ResultsTime = AsyncCallback->GetSolver()->GetPhysicsResultsTime_External();
	const double AsyncDt = AsyncCallback->GetSolver()->GetAsyncDeltaTime();

	TArray<Chaos::FCreatedModules> CombinedNewlyCreatedModuleGuids;
	Chaos::FCreatedModules ModulesEventData;
	CombinedNewlyCreatedModuleGuids.Init(ModulesEventData, CUVehicles.Num());

	// We need to pop new output data only if results time is beyond the latest cached output's step end time
	// Otherwise we can still use the ones we have stored in OutputRecord to interpolate
	// OutputRecord stores FChaosSimModuleManagerAsyncOutput which "InternalTime" si the time at the BEGINNING of the step that produced that output
	const double LatestOutputResultsTime = OutputRecord.GetLatestOutputStartTime() + AsyncDt;
	if (ResultsTime > LatestOutputResultsTime)
	{
		Chaos::TSimCallbackOutputHandle<FChaosSimModuleManagerAsyncOutput> NextAsyncOutput;
		while ((NextAsyncOutput = AsyncCallback->PopOutputData_External()))
		{
			// We process all events we find to never miss one
			for (int32 VehicleIdx = 0; VehicleIdx < NextAsyncOutput->VehicleOutputs.Num(); ++VehicleIdx)
			{
				TArray<Chaos::FCreatedModule>& NewlyCreatedModuleGuids = NextAsyncOutput->VehicleOutputs[VehicleIdx]->VehicleSimOutput.NewlyCreatedModuleGuids;
				if (NewlyCreatedModuleGuids.Num() > 0 && VehicleIdx < CombinedNewlyCreatedModuleGuids.Num())
				{
					CombinedNewlyCreatedModuleGuids[VehicleIdx].ModuleEvents.Append(NewlyCreatedModuleGuids);
				}
			}
			OutputRecord.ConsumeOutput(MoveTemp(NextAsyncOutput));
		}
	}

	FChaosSimModuleManagerAsyncOutput* PreviousOutput = OutputRecord.GetPreviousOutput();
	FChaosSimModuleManagerAsyncOutput* NextOutput = OutputRecord.GetNextOutput();
	UWorld* World = Scene.GetOwningWorld();
	FChaosSimModuleManagerAsyncInput* AsyncInput = AsyncCallback->GetProducerInputData_External();
	if (World && PreviousOutput && AsyncInput)
	{
		// InternalTime is the time at the beginning of the step, not at the end, whereas ResultsTime is the end time of the step (where we got to by taking the step)
		// OutputRecord deals with start times so we need to subtract AsyncDt from ResultsTime in what we pass to GetInterpolationFactor
		const double InterpolationBaseTime = ResultsTime - AsyncDt;
		float Alpha = OutputRecord.GetInterpolationFactor(InterpolationBaseTime);

		for (TWeakObjectPtr<UModularVehicleBaseComponent>& Vehicle : CUVehicles)
		{
			TStrongObjectPtr<UModularVehicleBaseComponent> StrongPtr = Vehicle.Pin();
			if (StrongPtr.IsValid())
			{
				StrongPtr->SetCurrentAsyncData(PreviousOutput, NextOutput, Alpha, Timestamp);
			}
		}
	}

	bool ForceSingleThread = !GSimModuleDebugParams.EnableMultithreading;
	{
		const TArray<TWeakObjectPtr<UModularVehicleBaseComponent>>& AwakeVehiclesBatch = CUVehicles;
		const TArray<Chaos::FCreatedModules>& ModuleEvents = CombinedNewlyCreatedModuleGuids;
		auto LambdaParallelUpdate2 = [&AwakeVehiclesBatch, &ModuleEvents](int32 Idx)
			{
				TWeakObjectPtr<UModularVehicleBaseComponent> Vehicle = AwakeVehiclesBatch[Idx];
				TStrongObjectPtr<UModularVehicleBaseComponent> StrongPtr = Vehicle.Pin();
				if (StrongPtr.IsValid())
				{
					StrongPtr->ParallelUpdate(ModuleEvents[Idx]); // gets output state from PT
				}
			};

		ParallelFor(AwakeVehiclesBatch.Num(), LambdaParallelUpdate2, ForceSingleThread);
	}
}