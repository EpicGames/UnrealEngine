// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ChaosSimModuleManagerAsyncCallback.h"

#include "ChaosModularVehicle/ModularVehicleBaseComponent.h"
#include "ChaosModularVehicle/ModularVehicleSimulationCU.h"
#include "ChaosModularVehicle/ModuleInputTokenStore.h"
#include "PBDRigidsSolver.h"
#include "ChaosModularVehicle/ModularVehicleBaseComponent.h"
#include "ChaosModularVehicle/ModuleInputTokenStore.h"
#include "GeometryCollection/GeometryCollectionParticlesData.h"
#include "PhysicsProxy/GeometryCollectionPhysicsProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "SimModule/ModuleFactoryRegister.h"
#include "Net/Core/NetBitArray.h"
#include "Net/Iris/ReplicationSystem/StructNetTokenDataStoreHelper.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosSimModuleManagerAsyncCallback)

FSimModuleDebugParams GSimModuleDebugParams;

DECLARE_CYCLE_STAT(TEXT("AsyncCallback:OnPreSimulate_Internal"), STAT_AsyncCallback_OnPreSimulate, STATGROUP_ChaosSimModuleManager);
DECLARE_CYCLE_STAT(TEXT("AsyncCallback:OnContactModification_Internal"), STAT_AsyncCallback_OnContactModification, STATGROUP_ChaosSimModuleManager);

namespace ChaosModularVehicleCVars
{
	bool bEnableStateReducedBandwidth = false;
	FAutoConsoleVariableRef EnableStateReducedBandwidth(TEXT("p.ModularVehicle.EnableStateReducedBandwidth"), bEnableStateReducedBandwidth, TEXT("Enable/Disable NetTokens and DeltaSerialization path for State of Modular Vehicles. Default: false"));
	bool bEnableInputReducedBandwidth = false;
	FAutoConsoleVariableRef EnableInputReducedBandwidth(TEXT("p.ModularVehicle.EnableInputReducedBandwidth"), bEnableInputReducedBandwidth, TEXT("Enable/Disable NetTokens and DeltaSerialization path for Input of Modular Vehicles. Default: false"));
	bool bEnableStateNetSerializeDebugPrinting = false;
	FAutoConsoleVariableRef EnableStateNetSerializeDebugPrinting(TEXT("p.ModularVehicle.EnableStateNetSerializeDebugPrinting"), bEnableStateNetSerializeDebugPrinting, TEXT("Enable/Disable debug logging during NetSerialization. Default: false"));
};

UE_NET_IMPLEMENT_NAMED_NETTOKEN_STRUCT_SERIALIZERS(ModuleInputNetTokenData)
UE_NET_IMPLEMENT_NAMED_NETTOKEN_STRUCT_SERIALIZERS(NetworkModularVehicleStateNetTokenData)

FName FChaosSimModuleManagerAsyncCallback::GetFNameForStatId() const
{
	const static FLazyName StaticName("FChaosSimModuleManagerAsyncCallback");
	return StaticName;
}

/**
 * Callback from Physics thread
 */

void FChaosSimModuleManagerAsyncCallback::ProcessInputs_Internal(int32 PhysicsStep)
{
	const FChaosSimModuleManagerAsyncInput* AsyncInput = GetConsumerInput_Internal();
	if (AsyncInput == nullptr)
	{
		return;
	}

	for (const TUniquePtr<FModularVehicleAsyncInput>& VehicleInput : AsyncInput->VehicleInputs)
	{
		VehicleInput->ProcessInputs();
	}
}

/**
 * Callback from Physics thread
 */
void FChaosSimModuleManagerAsyncCallback::OnPreSimulate_Internal()
{
	using namespace Chaos;

	SCOPE_CYCLE_COUNTER(STAT_AsyncCallback_OnPreSimulate);

	float DeltaTime = GetDeltaTime_Internal();
	float SimTime = GetSimTime_Internal();

	const FChaosSimModuleManagerAsyncInput* Input = GetConsumerInput_Internal();
	if (Input == nullptr)
	{
		return;
	}

	const int32 NumVehicles = Input->VehicleInputs.Num();

	UWorld* World = Input->World.Get();	//only safe to access for scene queries
	if (World == nullptr || NumVehicles == 0)
	{
		//world is gone so don't bother, or nothing to simulate.
		return;
	}

	Chaos::FPhysicsSolver* PhysicsSolver = static_cast<Chaos::FPhysicsSolver*>(GetSolver());
	if (PhysicsSolver == nullptr)
	{
		return;
	}

	FChaosSimModuleManagerAsyncOutput& Output = GetProducerOutputData_Internal();
	Output.VehicleOutputs.AddDefaulted(NumVehicles);
	Output.Timestamp = Input->Timestamp;

	const TArray<TUniquePtr<FModularVehicleAsyncInput>>& InputVehiclesBatch = Input->VehicleInputs;
	TArray<TUniquePtr<FModularVehicleAsyncOutput>>& OutputVehiclesBatch = Output.VehicleOutputs;

	// beware running the vehicle simulation in parallel, code must remain threadsafe
	auto LambdaParallelUpdate = [World, DeltaTime, SimTime, &InputVehiclesBatch, &OutputVehiclesBatch](int32 Idx)
	{
		QUICK_SCOPE_CYCLE_COUNTER(Stat_LambdaParallelUpdate);
		const FModularVehicleAsyncInput& VehicleInput = *InputVehiclesBatch[Idx];

		if (VehicleInput.Proxy == nullptr)
		{
			return;
		}

		bool bWake = false;
		OutputVehiclesBatch[Idx] = VehicleInput.Simulate(World, DeltaTime, SimTime, bWake);
		OutputVehiclesBatch[Idx]->Vehicle = VehicleInput.GetVehicle();
	};

	bool ForceSingleThread = !GSimModuleDebugParams.EnableMultithreading;
	PhysicsParallelFor(OutputVehiclesBatch.Num(), LambdaParallelUpdate, ForceSingleThread);

	// Delayed application of forces - This is separate from Simulate because forces cannot be executed multi-threaded
	for (const TUniquePtr<FModularVehicleAsyncInput>& VehicleInput : InputVehiclesBatch)
	{
		if (VehicleInput.IsValid())
		{
			VehicleInput->ApplyDeferredForces();
		}
	}
}

/**
 * Contact modification currently unused
 */
void FChaosSimModuleManagerAsyncCallback::OnContactModification_Internal(Chaos::FCollisionContactModifier& Modifications)
{
	using namespace Chaos;

	SCOPE_CYCLE_COUNTER(STAT_AsyncCallback_OnContactModification);

	float DeltaTime = GetDeltaTime_Internal();
	float SimTime = GetSimTime_Internal();

	const FChaosSimModuleManagerAsyncInput* Input = GetConsumerInput_Internal();
	if (Input == nullptr)
	{
		return;
	}

	const int32 NumVehicles = Input->VehicleInputs.Num();

	UWorld* World = Input->World.Get();	//only safe to access for scene queries
	if (World == nullptr || NumVehicles == 0)
	{
		//world is gone so don't bother.
		return;
	}

	Chaos::FPhysicsSolver* PhysicsSolver = static_cast<Chaos::FPhysicsSolver*>(GetSolver());
	if (PhysicsSolver == nullptr)
	{
		return;
	}

	const TArray<TUniquePtr<FModularVehicleAsyncInput>>& InputVehiclesBatch = Input->VehicleInputs;

	// beware running the vehicle simulation in parallel, code must remain threadsafe
	auto LambdaParallelUpdate = [&Modifications, &InputVehiclesBatch](int32 Idx)
	{
		const FModularVehicleAsyncInput& VehicleInput = *InputVehiclesBatch[Idx];

		if (VehicleInput.Proxy == nullptr)
		{
			return;
		}

		bool bWake = false;
		VehicleInput.OnContactModification(Modifications);

	};

	bool ForceSingleThread = !GSimModuleDebugParams.EnableMultithreading;
	PhysicsParallelFor(InputVehiclesBatch.Num(), LambdaParallelUpdate, ForceSingleThread);
}


TUniquePtr<FModularVehicleAsyncOutput> FModularVehicleAsyncInput::Simulate(UWorld* World, const float DeltaSeconds, const float TotalSeconds, bool& bWakeOut) const
{
	QUICK_SCOPE_CYCLE_COUNTER(Stat_FModularVehicleAsyncInput_Simulate);
	TUniquePtr<FModularVehicleAsyncOutput> Output = MakeUnique<FModularVehicleAsyncOutput>();

	//support nullptr because it allows us to go wide on filling the async inputs
	if (Proxy == nullptr)
	{
		return Output;
	}

	if (Vehicle)
	{
		if (FModularVehicleSimulation* Sim = Vehicle->VehicleSimulationPT.Get())
		{
			// FILL OUTPUT DATA HERE THAT WILL GET PASSED BACK TO THE GAME THREAD
			Sim->Simulate(World, DeltaSeconds, *this, *Output.Get(), Proxy);

			Chaos::FPhysicsSolver* Solver = Proxy->GetSolver<Chaos::FPhysicsSolver>();
			if (Solver && Solver->GetEvolution())
			{
				if (!Solver->GetEvolution()->IsResimming())
				{
					FModularVehicleAsyncOutput& OutputData = *Output.Get();
					Sim->FillOutputState(OutputData);
				}
			}
			else
			{
				FModularVehicleAsyncOutput& OutputData = *Output.Get();
				Sim->FillOutputState(OutputData);
			}
		}
	}


	Output->bValid = true;

	return MoveTemp(Output);
}

void FModularVehicleAsyncInput::OnContactModification(Chaos::FCollisionContactModifier& Modifications) const
{
	if (Vehicle && Vehicle->VehicleSimulationPT)
	{
		Vehicle->VehicleSimulationPT->OnContactModification(Modifications, Proxy);
	}
}

void FModularVehicleAsyncInput::ApplyDeferredForces() const
{
	if (Vehicle && Proxy && Vehicle->VehicleSimulationPT)
	{
		Vehicle->VehicleSimulationPT->ApplyDeferredForces(Proxy);
	}

}

void FModularVehicleAsyncInput::ProcessInputs()
{
	if (!GetVehicle())
	{
		return;
	}

	if (!GetVehicle()->VehicleSimulationPT)
	{
		return;
	}

	FModularVehicleSimulation* VehicleSim = GetVehicle()->VehicleSimulationPT.Get();

	if (VehicleSim == nullptr || !GetVehicle()->bUsingNetworkPhysicsPrediction || GetVehicle()->GetWorld() == nullptr)
	{
		return;
	}
	bool bIsResimming = false;
	if (FPhysScene* PhysScene = GetVehicle()->GetWorld()->GetPhysicsScene())
	{
		if (Chaos::FPhysicsSolver* LocalSolver = PhysScene->GetSolver())
		{
			bIsResimming = LocalSolver->GetEvolution()->IsResimming();
		}
	}

	if (bIsLocallyControlled && !bIsResimming)
	{
		VehicleSim->VehicleInputs = PhysicsInputs.NetworkInputs.VehicleInputs;
	}
	else
	{
		PhysicsInputs.NetworkInputs.VehicleInputs = VehicleSim->VehicleInputs;
	}
}

bool FNetworkModularVehicleInputs::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	static FNetworkModularVehicleInputs LocalDefaultDeltaSource;
	EModuleInputQuantizationType InputQuantizationType = EModuleInputQuantizationType::Default_16Bits;
	if (const UModularVehicleBaseComponent* ModularVehicleBaseComponent = Cast<UModularVehicleBaseComponent>(ImplementationComponent.Get()))
	{
		InputQuantizationType = ModularVehicleBaseComponent->InputQuantizationType;	
	}

	// If we should use delta source but the source is null, use a default source so that we at least consume the replicated bits correctly and doesn't cause a crash.
	if (bIsUsingDeltaSerialization && !DeltaSourceData)
	{
		DeltaSourceData = &LocalDefaultDeltaSource;
	}

	FNetworkPhysicsData::SerializeFrames(Ar);

	Ar.SerializeBits(&VehicleInputs.Reverse, 1);
	Ar.SerializeBits(&VehicleInputs.KeepAwake, 1);
	bOutSuccess = true;
	FNetworkModularVehicleInputs* DeltaSource = static_cast<FNetworkModularVehicleInputs*>(DeltaSourceData);
	if (ChaosModularVehicleCVars::bEnableInputReducedBandwidth && bIsUsingDeltaSerialization && DeltaSource)
	{
		using namespace UE::Net;
		bOutSuccess = true;
		
		TArray<FModuleInputValue>& InputValues = VehicleInputs.Container.AccessInputValues();
		const TArray<FModuleInputValue>& PreviousInputValues = DeltaSource->VehicleInputs.Container.AccessInputValues();
		
		FModuleInputNetTokenData InputStateData;
		InputStateData.Init(InputValues);
		const bool bNetTokenSuccess = TStructNetTokenDataStoreHelper<FModuleInputNetTokenData>::NetSerializeAndExportToken(Ar,Map,InputStateData);
		if (!bNetTokenSuccess)
		{
			bOutSuccess = false;
			return bOutSuccess;
		}
		
		uint32 Number = InputStateData.Types.Num();
		if (Ar.IsLoading())
		{
			InputValues.SetNum(Number);
		}

		for (uint32 I = 0; I < Number; I++)
		{
			InputValues[I].ConvertToType(static_cast<EModuleInputValueType>(InputStateData.Types[I]));
			InputValues[I].SetApplyInputDecay(InputStateData.DecayValues[I]);
			if (PreviousInputValues.Num() == InputValues.Num())
			{
				InputValues[I].DeltaNetSerialize(Ar, Map, bOutSuccess, PreviousInputValues[I], InputQuantizationType);
			}
			else
			{
				bOutSuccess = false;
				//Fail case.
				InputValues[I].DeltaNetSerialize(Ar, Map, bOutSuccess, InputValues[I], InputQuantizationType);
			}
		
		}
	}
	else
	{
		VehicleInputs.Container.Serialize(Ar, Map, bOutSuccess, InputQuantizationType);
	}
	
	return bOutSuccess;
}

void FNetworkModularVehicleInputs::ApplyData(UActorComponent* NetworkComponent) const
{
	if (GSimModuleDebugParams.EnableNetworkStateData)
	{
		if (UModularVehicleBaseComponent* ModularBaseComponent = Cast<UModularVehicleBaseComponent>(NetworkComponent))
		{
			if (FModularVehicleSimulation* VehicleSimulation = ModularBaseComponent->VehicleSimulationPT.Get())
			{
				VehicleSimulation->VehicleInputs = VehicleInputs;
			}
		}
	}
}

void FNetworkModularVehicleInputs::BuildData(const UActorComponent* NetworkComponent)
{
	if (GSimModuleDebugParams.EnableNetworkStateData)
	{
		if (const UModularVehicleBaseComponent* ModularBaseComponent = Cast<const UModularVehicleBaseComponent>(NetworkComponent))
		{
			if (const FModularVehicleSimulation* VehicleSimulation = ModularBaseComponent->VehicleSimulationPT.Get())
			{
				VehicleInputs = VehicleSimulation->VehicleInputs;
			}
		}
	}
}

void FNetworkModularVehicleInputs::InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData)
{
	const FNetworkModularVehicleInputs& MinInput = static_cast<const FNetworkModularVehicleInputs&>(MinData);
	const FNetworkModularVehicleInputs& MaxInput = static_cast<const FNetworkModularVehicleInputs&>(MaxData);

	const float LerpFactor = (LocalFrame - MinInput.LocalFrame) / (MaxInput.LocalFrame - MinInput.LocalFrame);

	VehicleInputs.Reverse = MinInput.VehicleInputs.Reverse;
	VehicleInputs.KeepAwake = MinInput.VehicleInputs.KeepAwake;
	VehicleInputs.Container.Lerp(MinInput.VehicleInputs.Container, MaxInput.VehicleInputs.Container, LerpFactor);
}

void FNetworkModularVehicleInputs::MergeData(const FNetworkPhysicsData& FromData)
{
	const FNetworkModularVehicleInputs& FromInput = static_cast<const FNetworkModularVehicleInputs&>(FromData);
	VehicleInputs.Container.Merge(FromInput.VehicleInputs.Container);
}

void FNetworkModularVehicleInputs::DecayData(float DecayAmount)
{
	VehicleInputs.Container.Decay(DecayAmount);
}

uint64 FNetworkModularVehicleStateNetTokenData::GetUniqueKey() const
{
	uint64 HashOfHashes = GetTypeHash(Hashes);
	uint64 HashOfIndexes = GetTypeHash(Indexes);
	uint64 HashOfShouldSerialize = GetTypeHash(ModuleShouldSerialize);
	return (HashOfHashes<<32) ^ HashOfIndexes ^ HashOfShouldSerialize;
}

void FNetworkModularVehicleStateNetTokenData::Init(const Chaos::FModuleNetDataArray& ModuleData)
{
	Hashes.Reset(ModuleData.Num());
	Indexes.Reset(ModuleData.Num());
	ModuleShouldSerialize.Reset(ModuleData.Num());
	for (int32 Idx = 0; Idx < ModuleData.Num(); Idx++)
	{
		uint32 Hash = Chaos::FModuleFactoryRegister::GetModuleHash(ModuleData[Idx]->GetSimType());
		Hashes.Add(Hash);
		Indexes.Add(ModuleData[Idx]->SimArrayIndex);
		ModuleShouldSerialize.Add(!ModuleData[Idx]->IsDefaultState());
	}
}

bool FNetworkModularVehicleStates::NetSerialize(FArchive& Ar, class UPackageMap* Map, bool& bOutSuccess)
{
	if (ChaosModularVehicleCVars::bEnableStateReducedBandwidth && bIsUsingDeltaSerialization)
	{
		return DeltaNetSerialize(Ar, Map, bOutSuccess);
	}
	
	using namespace UE::Net;

	FNetworkPhysicsData::SerializeFrames(Ar);

	uint32 NumNetModules = ModuleData.Num();
	Ar.SerializeIntPacked(NumNetModules);

	// Array of bits to mark which modules to serialize or not
	FNetBitArray ModulesBitArray(NumNetModules);

	if (Ar.IsLoading() && NumNetModules != ModuleData.Num())
	{
		ModuleData.Reserve(NumNetModules);
	}

	if (Ar.IsLoading())
	{
		Ar.SerializeBits(ModulesBitArray.GetData(), NumNetModules);

		if (NumNetModules != ModuleData.Num())
		{
			ModuleData.SetNum(NumNetModules);
		}

		for (uint32 I = 0; I < NumNetModules; I++)
		{
			uint32 ModuleTypeHash = 0;
			uint32 SimArrayIndexUnsigned = 0;

			Ar << ModuleTypeHash;
			Ar.SerializeIntPacked(SimArrayIndexUnsigned);

			const int32 SimArrayIndex = static_cast<int32>(SimArrayIndexUnsigned) - 1; // Convert back to signed and adjust

			if (TSharedPtr<Chaos::FModuleNetData> Data = Chaos::FModuleFactoryRegister::Get().GenerateNetData(ModuleTypeHash, SimArrayIndex))
			{
				check(ModuleTypeHash == Chaos::FModuleFactoryRegister::GetModuleHash(Data->GetSimType()));
				ModuleData[I] = Data;

				const bool bHasSerializedData = ModulesBitArray.IsBitSet(I);
				if (bHasSerializedData)
				{
					ModuleData[I]->Serialize(Ar);
				}
				else
				{
					ModuleData[I]->ApplyDefaultState();
				}
			}
		}
	}
	else
	{
		// Only mark modules for serialization if they are not in their default state
		for (uint32 I = 0; I < NumNetModules; I++)
		{
			if (ModuleData[I]->IsDefaultState() == false)
			{
				ModulesBitArray.SetBit(I);
			}
		}
		Ar.SerializeBits(ModulesBitArray.GetData(), NumNetModules);

		for (uint32 I = 0; I < NumNetModules; I++)
		{
			uint32 ModuleTypeHash = Chaos::FModuleFactoryRegister::GetModuleHash(ModuleData[I]->GetSimType());

			check(ModuleData[I]->SimArrayIndex + 1 >= 0);
			uint32 SimArrayIndexUnsigned = static_cast<uint32>(ModuleData[I]->SimArrayIndex + 1); // Convert to unsigned and align default -1 to 0. Done to be able to use SerializeIntPacked() for network optimization. 

			Ar << ModuleTypeHash;
			Ar.SerializeIntPacked(SimArrayIndexUnsigned);

			const bool bShouldSerializeData = ModulesBitArray.IsBitSet(I);
			if (bShouldSerializeData)
			{
				ModuleData[I]->Serialize(Ar);
			}
		}
	}

	bOutSuccess = true;
	return true;
}

bool FNetworkModularVehicleStates::DeltaNetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess)
{
	static FNetworkModularVehicleStates LocalDefaultDeltaSource;
	const bool bPrintDebugInfo = ChaosModularVehicleCVars::bEnableStateNetSerializeDebugPrinting;
	FBitReader* BitReader = (FBitReader*)&Ar;
	FBitWriter* BitWriter = (FBitWriter*)&Ar;
	using namespace UE::Net;

	// If we should use delta source but the source is null, use a default source so that we at least consume the replicated bits correctly and doesn't cause a crash.
	if (!DeltaSourceData)
	{
		DeltaSourceData = &LocalDefaultDeltaSource;
	}

	FNetworkPhysicsData::SerializeFrames(Ar);

	FNetworkModularVehicleStates* DeltaSource = static_cast<FNetworkModularVehicleStates*>(DeltaSourceData);
	UE_CLOGF(bPrintDebugInfo,LogModularVehicleSim, Warning, "====DeltaNetSerialize Saving: %d. ServerFrame: %d. DeltaSource_ServerFrame: %d Starting Bit: %lld", Ar.IsSaving(), ServerFrame, DeltaSource->ServerFrame, Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits());
	
	FNetworkModularVehicleStateNetTokenData VehicleStateData;
	VehicleStateData.Init(ModuleData);
	const bool bNetTokenSuccess = TStructNetTokenDataStoreHelper<FNetworkModularVehicleStateNetTokenData>::NetSerializeAndExportToken(Ar,Map,VehicleStateData);
	if (!bNetTokenSuccess)
	{
		bOutSuccess = false;
		return bOutSuccess;
	}

	auto GetDeltaDataHelper = [&](int32 InIdx, uint32 InModuleTypeHash, int32 InSimArrayIndex)
	{
		TSharedPtr<Chaos::FModuleNetData> DeltaData = nullptr;
		if (DeltaSource->ModuleData.IsValidIndex(InIdx))
		{
			const uint32 DeltaModuleHash = Chaos::FModuleFactoryRegister::GetModuleHash(DeltaSource->ModuleData[InIdx]->GetSimType());
			if (DeltaModuleHash == InModuleTypeHash)
			{
				DeltaData = DeltaSource->ModuleData[InIdx];
			}
		}
		if (DeltaData == nullptr)
		{
			UE_CLOGF(bPrintDebugInfo,LogModularVehicleSim, Warning, "==DeltaNetSerialize Generating Default Data for DeltaData Module %u", InModuleTypeHash);
			DeltaData = Chaos::FModuleFactoryRegister::Get().GenerateNetData(InModuleTypeHash, InSimArrayIndex);
			if (DeltaData == nullptr)
			{
				UE_LOGF(LogModularVehicleSim, Error, "Unable to generate net data for delta source when delta is invalid");
				bOutSuccess = false;
			}
			else
			{
				DeltaData->ApplyDefaultState();
			}
		}
		return DeltaData;
	};
	
	const uint32 NumNetModules = VehicleStateData.Hashes.Num();
	bOutSuccess = true;
	TMap<FName, uint32> SerializationStash;
	SerializationStash.Add(StashServerFrameKey, ServerFrame);
	if (Ar.IsLoading())
	{
		if (NumNetModules != ModuleData.Num())
		{
			ModuleData.SetNum(NumNetModules);
		}
		if(bPrintDebugInfo)
		{
			FString BitString = "";
			for (uint32 NetModuleIdx = 0; NetModuleIdx < NumNetModules; ++NetModuleIdx)
			{
				BitString += VehicleStateData.ModuleShouldSerialize[NetModuleIdx] ? "1" : "0";
			}
			UE_CLOGF(bPrintDebugInfo,LogModularVehicleSim, Warning, "==DeltaNetSerialize LOADING. Using ModuleShouldSerialize: %ls",*BitString);
		}
		for (uint32 I = 0; I < NumNetModules; I++)
		{
			const int32 SimArrayIndex = VehicleStateData.Indexes[I];
			const uint32 ModuleTypeHash = VehicleStateData.Hashes[I];
			
			if (TSharedPtr<Chaos::FModuleNetData> Data = Chaos::FModuleFactoryRegister::Get().GenerateNetData(ModuleTypeHash, SimArrayIndex))
			{
				int32 StartBit = Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits();
				UE_CLOGF(bPrintDebugInfo,LogModularVehicleSim, Warning, "==DeltaNetSerialize LOADING. ModuleData: %d STA. Bit: %lld",I,Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits());
				ModuleData[I] = Data;
				check(ModuleTypeHash == Chaos::FModuleFactoryRegister::GetModuleHash(Data->GetSimType()));
				if (VehicleStateData.ModuleShouldSerialize[I])
				{
					TSharedPtr<Chaos::FModuleNetData> DeltaData = GetDeltaDataHelper(I, ModuleTypeHash, SimArrayIndex);
					ModuleData[I]->DeltaSerializeWithStash(Ar, DeltaData.Get(),SerializationStash);
				}
				else
				{
					ModuleData[I]->ApplyDefaultState();
				}
				int32 EndBit = Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits();
				UE_CLOGF(bPrintDebugInfo,LogModularVehicleSim, Warning, "==DeltaNetSerialize LOADING. ModuleData: %d END. Bit: %lld Total: %d Error: %d",I,Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits(), EndBit-StartBit,Ar.IsError());
			}
		}
	}
	else
	{
		if(bPrintDebugInfo)
		{
			FString BitString = "";
			for (uint32 I = 0; I < NumNetModules; I++)
			{
				BitString += VehicleStateData.ModuleShouldSerialize[I] ? "1" : "0";
			}
			UE_CLOGF(bPrintDebugInfo,LogModularVehicleSim, Warning, "==DeltaNetSerialize SAVING. Using ModuleShouldSerialize: %ls",*BitString);
		}
		for (uint32 I = 0; I < NumNetModules; I++)
		{
			int32 StartBit = Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits();
			UE_CLOGF(bPrintDebugInfo,LogModularVehicleSim, Warning, "==DeltaNetSerialize SAVING. ModuleData: %d STA. Bit: %lld - %ls",I, Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits(), *ModuleData[I]->GetSimType().ToString());
			if (VehicleStateData.ModuleShouldSerialize[I])
			{
				const int32 SimArrayIndex = VehicleStateData.Indexes[I];
				const uint32 ModuleTypeHash = VehicleStateData.Hashes[I];
				TSharedPtr<Chaos::FModuleNetData> DeltaData = GetDeltaDataHelper(I, ModuleTypeHash, SimArrayIndex);
				ModuleData[I]->DeltaSerializeWithStash(Ar, DeltaData.Get(), SerializationStash);
			}
			int32 EndBit = Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits();
			UE_CLOGF(bPrintDebugInfo,LogModularVehicleSim, Warning, "==DeltaNetSerialize SAVING. ModuleData: %d END. Bit: %lld Size: %d - %ls",I, Ar.IsSaving()?BitWriter->GetNumBits():BitReader->GetPosBits(), EndBit-StartBit, *ModuleData[I]->GetSimType().ToString());
		}
	}
	
	return bOutSuccess;
}

void FNetworkModularVehicleStates::ApplyData(UActorComponent* NetworkComponent) const
{
	if (UModularVehicleBaseComponent* ModularBaseComponent = Cast<UModularVehicleBaseComponent>(NetworkComponent))
	{
		if (FModularVehicleSimulation* VehicleSimulation = ModularBaseComponent->VehicleSimulationPT.Get())
		{
			VehicleSimulation->AccessSimComponentTree()->SetSimState(ModuleData);
		}
	}
}

void FNetworkModularVehicleStates::BuildData(const UActorComponent* NetworkComponent)
{
	if (NetworkComponent)
	{
		if (const FModularVehicleSimulation* VehicleSimulation = Cast<const UModularVehicleBaseComponent>(NetworkComponent)->VehicleSimulationPT.Get())
		{
			VehicleSimulation->GetSimComponentTree()->SetNetState(ModuleData);
		}
	}
}

void FNetworkModularVehicleStates::InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData)
{
	const FNetworkModularVehicleStates& MinState = static_cast<const FNetworkModularVehicleStates&>(MinData);
	const FNetworkModularVehicleStates& MaxState = static_cast<const FNetworkModularVehicleStates&>(MaxData);

	const float LerpFactor = (LocalFrame - MinState.LocalFrame) / (MaxState.LocalFrame - MinState.LocalFrame);

	if (ModuleData.Num() != MinState.ModuleData.Num() || ModuleData.Num() != MaxState.ModuleData.Num())
	{
		UE_LOGF(LogModularVehicleSim, Error, "Mismatch in module data num when interpolating between min and max states!");
		return;
	}
	for (int I = 0; I < ModuleData.Num(); I++)
	{
		// if these don't match then something has gone terribly wrong
		check(ModuleData[I]->GetSimType() == MinState.ModuleData[I]->GetSimType());
		check(ModuleData[I]->GetSimType() == MaxState.ModuleData[I]->GetSimType());

		ModuleData[I]->Lerp(LerpFactor, *MinState.ModuleData[I].Get(), *MaxState.ModuleData[I].Get());
	}
}

