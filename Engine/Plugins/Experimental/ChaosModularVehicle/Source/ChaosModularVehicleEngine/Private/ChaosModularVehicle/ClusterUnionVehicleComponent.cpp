// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ClusterUnionVehicleComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClusterUnionVehicleComponent)

UClusterUnionVehicleComponent::UClusterUnionVehicleComponent(const FObjectInitializer& ObjectInitializer)
	: UClusterUnionComponent(ObjectInitializer)
{

}


bool UClusterUnionVehicleComponent::HasAnySockets() const
{
	if (!Sockets.IsEmpty())
	{
		return true;
	}

	//for (const TPair<TObjectKey<UPrimitiveComponent>, FClusteredComponentData>& Kvp : PerComponentData)
	//{
	//	if (UPrimitiveComponent* Component = Kvp.Key.ResolveObjectPtr())
	//	{
	//		if (Component->HasAnySockets())
	//		{
	//			return true;
	//		}
	//	}
	//}

	return Super::HasAnySockets();
}

bool UClusterUnionVehicleComponent::DoesSocketExist(FName InSocketName) const
{

	for (const FModularVehicleSocket& Socket : Sockets)
	{
		if (Socket.SocketName == InSocketName)
		{
			return true;
		}
	}

	//for (const TPair<TObjectKey<UPrimitiveComponent>, FClusteredComponentData>& Kvp : PerComponentData)
	//{
	//	if (UPrimitiveComponent* Component = Kvp.Key.ResolveObjectPtr())
	//	{
	//		if (Component->DoesSocketExist(InSocketName))
	//		{
	//			return true;
	//		}
	//	}
	//}

	return Super::DoesSocketExist(InSocketName);
}

FTransform UClusterUnionVehicleComponent::GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace) const
{
	for (const FModularVehicleSocket& Socket : Sockets)
	{
		if (Socket.SocketName == InSocketName)
		{
			FTransform SocketComponentSpaceTransform = Socket.GetLocalTransform();
			
			switch (TransformSpace)
			{
				case RTS_World:
					return SocketComponentSpaceTransform * GetComponentTransform();

				case RTS_Actor:
				{
					if (const AActor* Actor = GetOwner())
					{
						const FTransform SocketWorldSpaceTransform = SocketComponentSpaceTransform * GetComponentTransform();
						return SocketWorldSpaceTransform.GetRelativeTransform(Actor->GetTransform());
					}
					break;
				}

				case RTS_Component:
				{
					return SocketComponentSpaceTransform;
				}

				case RTS_ParentBoneSpace:
				default:
				{
					check(false);
				}
			}
		}
	}

	//for (const TPair<TObjectKey<UPrimitiveComponent>, FClusteredComponentData>& Kvp : PerComponentData)
	//{
	//	if (UPrimitiveComponent* Component = Kvp.Key.ResolveObjectPtr())
	//	{
	//		if (Component->DoesSocketExist(InSocketName))
	//		{
	//			return Component->GetSocketTransform(InSocketName, TransformSpace);
	//		}
	//	}
	//}

	return Super::GetSocketTransform(InSocketName, TransformSpace);

}

void UClusterUnionVehicleComponent::QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const
{
	for (const FModularVehicleSocket& Socket : Sockets)
	{
		FComponentSocketDescription& Desc = OutSockets.AddZeroed_GetRef();
		Desc.Name = Socket.SocketName;
		Desc.Type = EComponentSocketType::Type::Socket;
	}


	//for (const TPair<TObjectKey<UPrimitiveComponent>, FClusteredComponentData>& Kvp : PerComponentData)
	//{
	//	if (UPrimitiveComponent* Component = Kvp.Key.ResolveObjectPtr())
	//	{
	//		Component->QuerySupportedSockets(OutSockets);
	//	}
	//}
}
