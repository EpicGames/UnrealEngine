// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/InputModifier_ModularVehicleSmooth.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputModifier_ModularVehicleSmooth)

FInputActionValue UInputModifier_ModularVehicleSmooth::ModifyRaw_Implementation(
	const UEnhancedPlayerInput* PlayerInput, 
	FInputActionValue CurrentValue, 
	float DeltaTime)
{
	// Only handle scalar (float) inputs here.
	float Current = CurrentValue.Get<float>();

	// No Fall rate just snaps to zero
	if (FMath::Abs(Current) < SMALL_NUMBER)
	{
		PreviousValue = 0.0f;
		return 0.0f;
	}

	float DeltaValue = Current - PreviousValue;

	const float MaxDeltaValue = DeltaTime * RiseRate;
	const float ClampedDeltaValue = FMath::Clamp(DeltaValue, -MaxDeltaValue, MaxDeltaValue);
	
	float SmoothedValue = PreviousValue + ClampedDeltaValue;
	PreviousValue = SmoothedValue;

	return SmoothedValue;
}
