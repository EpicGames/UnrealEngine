// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/InputProducer.h"

// Default input producer

#include UE_INLINE_GENERATED_CPP_BY_NAME(InputProducer)

void UVehicleDefaultInputProducer::InitializeContainer(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut, EModuleInputQuantizationType InInputQuantizationType)
{
	Super::InitializeContainer(SetupData, NameMapOut, InInputQuantizationType);

	MergedInput.Initialize(SetupData, NameMapOut);
}

void UVehicleDefaultInputProducer::BufferInput(const FInputNameMap& InNameMap, const FName InName, const FModuleInputValue& InValue, EModuleInputBufferActionType BufferAction)
{
	//UE_LOGF(LogTemp, Warning, "BufferInput - InName %ls Val=%ls", *InName.ToString(), *InValue.ToString());

	switch(BufferAction)
	{
	case EModuleInputBufferActionType::Override:
		{
			// inputs are merged here rather than buffered, since they would be merged anyway before use in ProduceInput
			FInputInterface Inputs(InNameMap, MergedInput, InputQuantizationType);
			Inputs.MergeValue(InName, InValue);
			break;
		}
	case EModuleInputBufferActionType::Combine:
		{
			FInputInterface Inputs(InNameMap, MergedInput, InputQuantizationType);
			Inputs.CombineValue(InName, InValue);
			break;
		}
	case EModuleInputBufferActionType::Average:
		{
			FInputInterface Inputs(InNameMap, MergedInput, InputQuantizationType);
			Inputs.AverageValue(InName, InValue);
			break;
		}
	default:
		break;
	}
}

void UVehicleDefaultInputProducer::ProduceInput(int32 PhysicsStep, int32 NumSteps, const FInputNameMap& InNameMap, FModuleInputContainer& InOutContainer)
{
	//UE_LOGF(LogTemp, Warning, "ProduceInput - PhysicsStep %d, NumSteps %d", PhysicsStep, NumSteps);
	//FInputInterface Inputs(InNameMap, MergedInput);
	//UE_LOGF(LogTemp, Warning, ".. Throttle %ls", *Inputs.GetValue("Throttle").ToString());

	// copy state out
	InOutContainer = MergedInput;

	// reset state for next frame
	MergedInput.ZeroValues();
}


// Example Playback Input Producer

void UVehiclePlaybackInputProducer::InitializeContainer(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut, EModuleInputQuantizationType InInputQuantizationType)
{
	Super::InitializeContainer(SetupData, NameMapOut, InInputQuantizationType);

	int32 Seed = 123;
	FRandomStream Random(Seed);
	StartStep = 0;

	// Initialize a single container once
	FModuleInputContainer InputsForFrame;
	InputsForFrame.Initialize(SetupData, NameMapOut);

	PlaybackBuffer.Reserve(BufferLength);
	for (int32 I = 0; I < BufferLength; I++)
	{
		// copy initialized container into array
		PlaybackBuffer.Emplace(InputsForFrame);

		// change value of some intputs
		FInputInterface Inputs(NameMapOut, PlaybackBuffer[I], InputQuantizationType);
		Inputs.SetValue("Throttle", Random.FRand());
		Inputs.SetValue("Steering", 1.0f - 2.0f * Random.FRand());
	}

}

void UVehiclePlaybackInputProducer::BufferInput(const FInputNameMap& InNameMap, const FName InName, const FModuleInputValue& InValue, EModuleInputBufferActionType BufferAction)
{
	// NOP
}

void UVehiclePlaybackInputProducer::ProduceInput(int32 PhysicsStep, int32 NumSteps, const FInputNameMap& InNameMap, FModuleInputContainer& InOutContainer)
{
	if (StartStep == 0)
	{
		StartStep = PhysicsStep;
	}

	int32 UseIndex = PhysicsStep - StartStep + NumSteps - 1;
	if (UseIndex >=0 && UseIndex < PlaybackBuffer.Num())
	{
		FModuleInputContainer& UseContainer = PlaybackBuffer[UseIndex];
		InOutContainer = UseContainer;
	}
	else
	{
		InOutContainer.ZeroValues();
		StartStep = PhysicsStep;	// restart loop
	}

}


// Example Random Input Producer

void UVehicleRandomInputProducer::InitializeContainer(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut, EModuleInputQuantizationType InInputQuantizationType)
{
	Super::InitializeContainer(SetupData, NameMapOut, InInputQuantizationType);

	PlaybackContainer.Initialize(SetupData, NameMapOut);
}

void UVehicleRandomInputProducer::BufferInput(const FInputNameMap& InNameMap, const FName InName, const FModuleInputValue& InValue, EModuleInputBufferActionType BufferAction)
{
	// NOP
}

void UVehicleRandomInputProducer::ProduceInput(int32 PhysicsStep, int32 NumSteps, const FInputNameMap& InNameMap, FModuleInputContainer& InOutContainer)
{
	int32 Seed = 123;
	static FRandomStream Random(Seed);

	// new control settings generated every ChangeInputFrequency number of frames (every frame is too quick)
	// previous controls are held in the PlaybackContainer between the changes
	if (PhysicsStep % ChangeInputFrequency == 0)
	{
		// clear old input
		PlaybackContainer.ZeroValues();

		// generate new random input
		FInputInterface Inputs(InNameMap, PlaybackContainer, InputQuantizationType);
		Inputs.SetValue("Throttle", Random.FRand());
		Inputs.SetValue("Steering", 1.0f - 2.0f * Random.FRand());
	}

	InOutContainer = PlaybackContainer;

}

