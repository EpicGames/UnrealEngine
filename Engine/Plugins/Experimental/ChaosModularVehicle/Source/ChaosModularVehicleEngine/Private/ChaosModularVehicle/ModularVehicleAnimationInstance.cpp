// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	UModularVehicleAnimationInstance.cpp: Single Node Tree Instance 
	Only plays one animation at a time. 
=============================================================================*/ 

#include "ChaosModularVehicle/ModularVehicleAnimationInstance.h"
#include "ChaosModularVehicle/ModularVehicleBaseComponent.h"
#include "ChaosModularVehicle/ModularVehicleClusterPawn.h"
#include "AnimationRuntime.h"

/////////////////////////////////////////////////////
// UModularVehicleAnimationInstance
/////////////////////////////////////////////////////

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularVehicleAnimationInstance)

UModularVehicleAnimationInstance::UModularVehicleAnimationInstance(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

class AModularVehicleClusterPawn* UModularVehicleAnimationInstance::GetVehicle()
{
	return Cast<AModularVehicleClusterPawn>(GetOwningActor());
}

void UModularVehicleAnimationInstance::NativeInitializeAnimation()
{
	// Find a wheeled movement component
	if (AActor* Actor = GetOwningActor())
	{
		if (UModularVehicleBaseComponent* FoundModularVehicleComponent = Actor->FindComponentByClass<UModularVehicleBaseComponent>())
		{
			SetModularVehicleComponent(FoundModularVehicleComponent);
		}
	}
}

FAnimInstanceProxy* UModularVehicleAnimationInstance::CreateAnimInstanceProxy()
{
	return &AnimInstanceProxy;
}

void UModularVehicleAnimationInstance::DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy)
{
}

/////////////////////////////////////////////////////
//// PROXY ///
/////////////////////////////////////////////////////

void FModularVehicleAnimationInstanceProxy::SetModularVehicleComponent(const UModularVehicleBaseComponent* InWheeledVehicleComponent)
{
	const UModularVehicleBaseComponent* ModularVehicleComponent = InWheeledVehicleComponent;

	//initialize data
	// 
	// take setup from modular vehicle and create anim instance data
	const TArray<FModuleAnimationSetup>& ModuleAnimationSetups = ModularVehicleComponent->GetModuleAnimationSetups();

	const int32 NumOfModules = ModuleAnimationSetups.Num();
	ModuleInstances.Empty(NumOfModules);
	if (NumOfModules > 0)
	{
		ModuleInstances.AddZeroed(NumOfModules);
			
		// now add Module data
		for (int32 ModuleIndex = 0; ModuleIndex < ModuleInstances.Num(); ++ModuleIndex)
		{
			FModuleAnimationData& ModuleInstance = ModuleInstances[ModuleIndex];
			const FModuleAnimationSetup& ModuleSetup = ModuleAnimationSetups[ModuleIndex];

			// set data
			ModuleInstance.BoneName = ModuleSetup.BoneName;
			ModuleInstance.LocOffset = FVector::ZeroVector;
			ModuleInstance.RotOffset = FRotator::ZeroRotator;
		}
	}
}

void FModularVehicleAnimationInstanceProxy::PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds)
{
	Super::PreUpdate(InAnimInstance, DeltaSeconds);

	if (const UModularVehicleAnimationInstance* VehicleAnimInstance = CastChecked<UModularVehicleAnimationInstance>(InAnimInstance))
	{
		// more modules have been added during runtime
		if (const UModularVehicleBaseComponent* ModularVehicleComponent = VehicleAnimInstance->GetModularVehicleComponent())
		{
			if (ModuleInstances.Num() < ModularVehicleComponent->GetModuleAnimationSetups().Num())
			{
				int NumNew = ModularVehicleComponent->GetModuleAnimationSetups().Num() - ModuleInstances.Num();
				int StartIdx = ModuleInstances.Num();
				for (int I = 0; I < NumNew; I++)
				{
					FModuleAnimationData ModuleInstance;
					ModuleInstance.BoneName = ModularVehicleComponent->GetModuleAnimationSetups()[StartIdx + I].BoneName;
					ModuleInstance.LocOffset = FVector::ZeroVector;
					ModuleInstance.RotOffset = FRotator::ZeroRotator;

					ModuleInstances.Add(ModuleInstance);
				}
			}
		}

		if (const UModularVehicleBaseComponent* ModularVehicleComponent = VehicleAnimInstance->GetModularVehicleComponent())
		{
			for (int32 ModuleIndex = 0; ModuleIndex < ModuleInstances.Num(); ++ModuleIndex)
			{
				FModuleAnimationData& ModuleInstance = ModuleInstances[ModuleIndex];
				if (ModularVehicleComponent->GetModuleAnimationSetups().IsValidIndex(ModuleIndex))
				{ 
					const FModuleAnimationSetup& ModuleAnim = ModularVehicleComponent->GetModuleAnimationSetups()[ModuleIndex];
					{
						ModuleInstance.LocOffset = ModuleAnim.LocOffset;
						ModuleInstance.RotOffset = ModuleAnim.RotOffset;
						ModuleInstance.Flags |= ModuleAnim.AnimFlags;
					}
				}
			}
		}
	}
}

