// Copyright Epic Games, Inc. All Rights Reserved.

/*=============================================================================
	ModularVehicleAsset.cpp: UModularVehicleAsset methods.
=============================================================================*/
#include "ChaosModularVehicle/ModularVehicleAsset.h"
#include "ChaosModularVehicle/ModularSimCollection.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularVehicleAsset)

DEFINE_LOG_CATEGORY_STATIC(LogModularVehicleAssetInternal, Log, All);

FModularVehicleAssetEdit::FModularVehicleAssetEdit(UModularVehicleAsset* InAsset)
	: Asset(InAsset)
{
}

FModularVehicleAssetEdit::~FModularVehicleAssetEdit()
{
}

UModularVehicleAsset* FModularVehicleAssetEdit::GetAsset()
{
	if (Asset)
	{
		return Asset;
	}
	return nullptr;
}

UModularVehicleAsset::UModularVehicleAsset(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, ModularSimCollection(new FModularSimCollection())
{
}


/** Serialize */
void UModularVehicleAsset::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	bool bCreateSimulationData = false;
	Chaos::FChaosArchive ChaosAr(Ar);
	ModularSimCollection->Serialize(ChaosAr);
}
