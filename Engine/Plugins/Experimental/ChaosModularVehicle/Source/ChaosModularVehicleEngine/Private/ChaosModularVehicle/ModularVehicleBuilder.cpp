// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ModularVehicleBuilder.h"
#include "ChaosModularVehicle/ModularVehicleBaseComponent.h"
#include "ChaosModularVehicle/VehicleSimBaseComponent.h"
#include "ChaosModularVehicle/ModularVehicleSimulationCU.h"
#include "SimModule/SimModulesInclude.h"


void FModularVehicleBuilder::GenerateSimTree(UModularVehicleBaseComponent* ModularVehicle)
{
	if (ModularVehicle)
	{
		bool RequiresAnimation = true;
		if (ModularVehicle->GetOwner())
		{
			RequiresAnimation = ModularVehicle->GetOwner()->GetNetMode() != NM_DedicatedServer;
		}

		// take the UVehicleSimBaseComponents and pack them into a more compact structure for simulation on physics callback thread.
		if (TUniquePtr<Chaos::FSimModuleTree> SimModuleTree = MakeUnique<Chaos::FSimModuleTree>()) // create physics output container
		{
			SimModuleTree->SetAnimationEnabled(RequiresAnimation);

			SimModuleTree->SetSimTreeProcessingOrder(ModularVehicle->GetTreeProcessingOrder());

			// Physics thread takes ownership of the tree from here
			ModularVehicle->VehicleSimulationPT->Initialize(SimModuleTree);
		}
	}
}


void FModularVehicleBuilder::FixupTreeLinks(TUniquePtr<Chaos::FSimModuleTree>& SimModuleTree)
{
	using namespace Chaos;

	if (SimModuleTree.IsValid())
	{
		const FSimModuleTree::FSimModuleNode* TransmissionNode = SimModuleTree->LocateNodeByType<Chaos::FTransmissionSimModule>();

		for (int I = 0; I < SimModuleTree->NumActiveNodes(); I++)
		{
			// link suspension and wheel references to one another
			ISimulationModuleBase* Module = SimModuleTree->AccessSimModule(I);
			if (Module && Module->IsSimType<FSuspensionBaseInterface>())
			{
				FSimModuleTree::FSimModuleNode& SuspensionNode = SimModuleTree->GetNode(I);

				if (SuspensionNode.Parent != FSimModuleTree::FSimModuleNode::INVALID_IDX)
				{
					FSimModuleTree::FSimModuleNode& SuspensionParentNode = SimModuleTree->GetNode(SuspensionNode.Parent);

					ISimulationModuleBase* ParentModule = SimModuleTree->AccessSimModule(SuspensionParentNode.SimModule->GetTreeIndex());

					if (FWheelBaseInterface* Wheel = ParentModule->Cast<FWheelBaseInterface>())
					{
						if (FSuspensionBaseInterface* Suspension = Module->Cast<FSuspensionBaseInterface>())
						{
							Wheel->SetSuspensionSimTreeIndex(Suspension->GetTreeIndex());
							Suspension->SetWheelSimTreeIndex(Wheel->GetTreeIndex());
						}
					}
				}

				for (int Child : SuspensionNode.Children)
				{
					if (Child != FSimModuleTree::FSimModuleNode::INVALID_IDX)
					{
						FSimModuleTree::FSimModuleNode& SuspensionChildNode = SimModuleTree->GetNode(Child);

						ISimulationModuleBase* ChildModule = SimModuleTree->AccessSimModule(SuspensionChildNode.SimModule->GetTreeIndex());

						if (FWheelBaseInterface* Wheel = ChildModule->Cast<FWheelBaseInterface>())
						{
							if (FSuspensionBaseInterface* Suspension = Module->Cast<FSuspensionBaseInterface>())
							{
								Wheel->SetSuspensionSimTreeIndex(Suspension->GetTreeIndex());
								Suspension->SetWheelSimTreeIndex(Wheel->GetTreeIndex());
							}
						}
					}
				}
			}

			// temporarily - link suspension/wheels to engine/clutch/transmission, assuming only 1 transmission & all wheels are powered
			if (TransmissionNode)
			{
				if (Module && Module->IsSimType<class FWheelSimModule>())
				{
					SimModuleTree->Reparent(Module->GetTreeIndex(), TransmissionNode->SimModule->GetTreeIndex());
				}
			}

		}
	}
}


