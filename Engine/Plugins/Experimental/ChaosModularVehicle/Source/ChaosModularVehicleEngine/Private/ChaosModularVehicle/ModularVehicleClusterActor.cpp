// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ModularVehicleClusterActor.h"
#include "ChaosModularVehicle/ClusterUnionVehicleComponent.h"
#include "ChaosModularVehicle/ModularVehicleBaseComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularVehicleClusterActor)


AModularVehicleClusterActor::AModularVehicleClusterActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ClusterUnionVehicleComponent = CreateDefaultSubobject<UClusterUnionVehicleComponent>(TEXT("ClusterUnionVehicleComponent0"));
	SetRootComponent(ClusterUnionVehicleComponent);

	// Vehicle sim piggy-backs off of the cluster union component & its events
	VehicleSimComponent = CreateDefaultSubobject<UModularVehicleBaseComponent>(TEXT("VehicleSimComponent0"));
	VehicleSimComponent->SetClusterComponent(ClusterUnionVehicleComponent);
	VehicleSimComponent->bAutoRegisterUpdatedComponent = false;
                          
	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	SetReplicatingMovement(true);
}
