// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ModularVehicleClusterPawn.h"
#include "ChaosModularVehicle/ClusterUnionVehicleComponent.h"
#include "ChaosModularVehicle/ModularVehicleBaseComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularVehicleClusterPawn)


AModularVehicleClusterPawn::AModularVehicleClusterPawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ClusterUnionVehicleComponent = CreateDefaultSubobject<UClusterUnionVehicleComponent>(TEXT("ClusterUnionVehicleComponent0"));
	SetRootComponent(ClusterUnionVehicleComponent);

	// Vehicle sim piggy-backs off of the cluster union component & its events
	VehicleSimComponent = CreateDefaultSubobject<UModularVehicleBaseComponent>(TEXT("VehicleSimComponent0"));
	VehicleSimComponent->SetClusterComponent(ClusterUnionVehicleComponent);

	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	SetReplicatingMovement(true);
}
