// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ModularVehicleSimulationCU.h"

#include "ChaosModularVehicle/ModularVehicleDefaultAsyncInput.h"
#include "SimModule/SimModulesInclude.h"
#include "SimModule/ModuleInput.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PhysicsProxy/SingleParticlePhysicsProxy.h"
#include "PhysicalMaterials/PhysicalMaterial.h"
#include "PhysicsEngine/PhysicsSettings.h"
#include "Chaos/ClusterUnionManager.h"
#include "Chaos/DebugDrawQueue.h"
#include "Chaos/PhysicsObjectCollisionInterface.h"
#include "Chaos/PhysicsObjectInternalInterface.h"
#include "Engine/World.h"
#include "Engine/HitResult.h"
#include "ChaosModularVehicle/ModularVehicleDebug.h"
#include "Components/PrimitiveComponent.h"
#include "Engine/OverlapResult.h"
#include "Runtime/Experimental/Chaos/Private/Chaos/PhysicsObjectInternal.h"

FModularVehicleDebugParams GModularVehicleDebugParams;
DEFINE_LOG_CATEGORY(LogModularVehicleSim);

bool bModularVehicle_DumpModuleTreeStructure_Enabled = false;
FAutoConsoleVariableRef CVarModularVehicleDumpModuleTreeStructureEnabled2(TEXT("p.ModularVehicle.DumpModuleTreeStructure.Enabled"), bModularVehicle_DumpModuleTreeStructure_Enabled, TEXT("Enable/Disable logging of module tree structure every time there is a change."));

FAutoConsoleVariableRef CVarChaosModularVehiclesRaycastsEnabled(TEXT("p.ModularVehicle.SuspensionRaycastsEnabled"), GModularVehicleDebugParams.SuspensionRaycastsEnabled, TEXT("Enable/Disable Suspension Raycasts."));

#if CHAOS_DEBUG_DRAW
FAutoConsoleVariableRef CVarChaosModularVehiclesShowRaycasts(TEXT("p.ModularVehicle.ShowSuspensionRaycasts"), GModularVehicleDebugParams.ShowSuspensionRaycasts, TEXT("Enable/Disable Suspension Raycast Visualisation."));
FAutoConsoleVariableRef CVarChaosModularVehiclesShowWheelData(TEXT("p.ModularVehicle.ShowWheelData"), GModularVehicleDebugParams.ShowWheelData, TEXT("Enable/Disable Displaying Wheel Simulation Data."));
FAutoConsoleVariableRef CVarChaosModularVehiclesShowRaycastMaterial(TEXT("p.ModularVehicle.ShowRaycastMaterial"), GModularVehicleDebugParams.ShowRaycastMaterial, TEXT("Enable/Disable Raycast Material Hit Visualisation."));
FAutoConsoleVariableRef CVarChaosModularVehiclesShowWheelCollisionNormal(TEXT("p.ModularVehicle.ShowWheelCollisionNormal"), GModularVehicleDebugParams.ShowWheelCollisionNormal, TEXT("Enable/Disable Wheel Collision Normal Visualisation."));
FAutoConsoleVariableRef CVarChaosModularVehiclesFrictionOverride(TEXT("p.ModularVehicle.FrictionOverride"), GModularVehicleDebugParams.FrictionOverride, TEXT("Override the physics material friction value.."));
FAutoConsoleVariableRef CVarChaosModularVehiclesDisableAnim(TEXT("p.ModularVehicle.DisableAnim"), GModularVehicleDebugParams.DisableAnim, TEXT("Disable animating wheels, etc"));
#endif

void FModularVehicleSimulation::Initialize(TUniquePtr<Chaos::FSimModuleTree>& InSimModuleTree)
{
	UE_LOGF(LogModularVehicleSim, Log, "FModularVehicleSimulation::Initialize");

	SimModuleTree = MoveTemp(InSimModuleTree);
}

void FModularVehicleSimulation::Terminate()
{
	UE_LOGF(LogModularVehicleSim, Log, "FModularVehicleSimulation::Terminate");

	RootParticle = nullptr;
	SimModuleTree.Reset(nullptr);
}

void FModularVehicleSimulation::Simulate(UWorld* InWorld, float DeltaSeconds, const FModularVehicleAsyncInput& InputData, FModularVehicleAsyncOutput& OutputData, IPhysicsProxyBase* Proxy)
{
	CacheRootParticle(Proxy);

	ActionTreeUpdates();

	Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(Proxy->GetSolver<Chaos::FPhysicsSolver>());
	int CurrentFrame = -1;
	if (RigidsSolver != nullptr)
	{
		Chaos::FRewindData* RewindData = RigidsSolver->GetRewindData();
		if (RewindData != nullptr)
		{
			CurrentFrame = RewindData->CurrentFrame();
		}
	}

	SimulateModuleTree(InWorld, DeltaSeconds, InputData, OutputData, Proxy);
}

void FModularVehicleSimulation::OnContactModification(Chaos::FCollisionContactModifier& Modifier, IPhysicsProxyBase* Proxy)
{
	using namespace Chaos;
	Chaos::EnsureIsInPhysicsThreadContext();

	if (SimModuleTree.IsValid())
	{
		SimModuleTree->OnContactModification(Modifier, Proxy);
	}
}

void FModularVehicleSimulation::SimulateModuleTree(UWorld* InWorld, float DeltaSeconds, const FModularVehicleAsyncInput& InputData, FModularVehicleAsyncOutput& OutputData, IPhysicsProxyBase* Proxy)
{
	Chaos::EnsureIsInPhysicsThreadContext();

	if (Proxy && SimModuleTree.IsValid())
	{
		int InitialNum = SimModuleTree->GetSimulationModuleTree().Num();
		if (InitialNum == 0)
		{
			return;
		}
		//if (InWorld)
		//{
		//	WriteNetReport(InWorld->IsNetMode(NM_Client), FString::Printf(TEXT("X %s,  R %s,  V %s,  W %s")
		//		, *Proxy->GetParticle_Internal()->X().ToString()
		//		, *Proxy->GetParticle_Internal()->R().ToString()
		//		, *Proxy->GetParticle_Internal()->V().ToString()
		//		, *Proxy->GetParticle_Internal()->W().ToString()));
		//}

		UE::TReadScopeLock InputConfigLock(InputConfigurationLock);

		FModuleInputContainer Container = InputData.PhysicsInputs.NetworkInputs.VehicleInputs.Container;

		if (ImplementsTestBuffer())
		{
			Chaos::FPBDRigidsSolver* RigidsSolver = static_cast<Chaos::FPBDRigidsSolver*>(Proxy->GetSolver<Chaos::FPhysicsSolver>());
			check(RigidsSolver);
			const int32 CurrentPhysicsFrame = RigidsSolver->GetCurrentFrame();

			if (TestInputBufferStartFrame < 0)
			{
				TestInputBufferStartFrame = CurrentPhysicsFrame;
			}

			if (TestInputBufferStartFrame <= CurrentPhysicsFrame)
			{
				int32 InputFrame = (CurrentPhysicsFrame - TestInputBufferStartFrame);

				if (ImplementsLoopingTestBuffer() && !TestInputBuffer.IsValidIndex(InputFrame))
				{
					TestInputBufferStartFrame = CurrentPhysicsFrame;
					InputFrame = 0;
				}

				if (TestInputBuffer.IsValidIndex(InputFrame))
				{
					Container = TestInputBuffer[InputFrame];
				}
			}
		}

		FInputInterface InputInterface(InputNameMap, Container, InputQuantizationType);

		FModuleInputContainer StateInputContainer = InputData.PhysicsInputs.StateInputs.StateInputContainer;
		FInputInterface StateInterface(StateNameMap, StateInputContainer, InputQuantizationType);

		SimInputData.ControlInputs = &InputInterface;
		SimInputData.StateInputs = &StateInterface;
		SimInputData.bKeepVehicleAwake = InputData.PhysicsInputs.NetworkInputs.VehicleInputs.KeepAwake;
		SimInputData.CurrentTimeDilation = InputData.PhysicsInputs.CurrentTimeDilation;


		if(SimModuleTree->GetSimTreeProcessingOrder() != ESimTreeProcessingOrder::ManualOverride)
		{
			PerformAdditionalSimWork(DeltaSeconds, InWorld, InputData, Proxy, SimInputData);
		}
		// run the dynamics simulation, engine, suspension, wheels, aerofoils etc.
		SimModuleTree->Simulate(DeltaSeconds, SimInputData, Proxy, RootParticle);

		if(SimModuleTree->GetSimTreeProcessingOrder() == ESimTreeProcessingOrder::ManualOverride)
		{
			VehicleSimulationCallback.Broadcast(this, InWorld, DeltaSeconds, InputData, SimInputData, Proxy, RootParticle, SimModuleTree.Get());
		}

		// Clear those Inputs that we don't want to remain set if the physics simulation thread ticks more frames than the GT
		InputData.PhysicsInputs.NetworkInputs.VehicleInputs.Container.ClearConsumedInputs();
		InputData.PhysicsInputs.StateInputs.StateInputContainer.ClearConsumedInputs();
	}

}

void FModularVehicleSimulation::CacheRootParticle(IPhysicsProxyBase* Proxy)
{
	Chaos::EnsureIsInPhysicsThreadContext();
	using namespace Chaos;
	RootParticle = nullptr;

	if (Proxy == nullptr)
	{
		return;
	}

	switch (Proxy->GetType())
	{
		case EPhysicsProxyType::ClusterUnionProxy:
		{
			if (FClusterUnionPhysicsProxy* CUProxy = static_cast<FClusterUnionPhysicsProxy*>(Proxy))
			{
				FPBDRigidsEvolutionGBF& Evolution = *static_cast<FPBDRigidsSolver*>(CUProxy->GetSolver<FPBDRigidsSolver>())->GetEvolution();
				FClusterUnionManager& ClusterUnionManager = Evolution.GetRigidClustering().GetClusterUnionManager();
				const FClusterUnionIndex& CUI = CUProxy->GetClusterUnionIndex();
				if (FClusterUnion* ClusterUnion = ClusterUnionManager.FindClusterUnion(CUI))
				{
					if (FPBDRigidClusteredParticleHandle* ClusterHandle = ClusterUnion->InternalCluster)
					{
						RootParticle = ClusterHandle;
					}
				}
			}
		}
		break;

		case EPhysicsProxyType::SingleParticleProxy:
		{
			if (FSingleParticlePhysicsProxy* ParticleProxy = static_cast<FSingleParticlePhysicsProxy*>(Proxy))
			{
				RootParticle = ParticleProxy->GetHandle_LowLevel() ? ParticleProxy->GetHandle_LowLevel()->CastToRigidParticle() : nullptr;
			}
		}
		break;

		default:
		{
			UE_LOGF(LogModularVehicleSim, Error, "Unsupported Particle type");
		}
		break;
	}
}

void FModularVehicleSimulation::PerformAdditionalSimWork(float DeltaSeconds, UWorld* InWorld, const FModularVehicleAsyncInput& InputData, IPhysicsProxyBase* Proxy, Chaos::FAllInputs& AllInputs)
{
	using namespace Chaos;
	check(Proxy);
	Chaos::EnsureIsInPhysicsThreadContext();


	// clear all ground interactions
	if (SimModuleTree)
	{
		const TArray<FSimModuleTree::FSimModuleNode>& ModuleArray = SimModuleTree->GetSimulationModuleTree();

		for (const FSimModuleTree::FSimModuleNode& Node : ModuleArray)
		{
			if (Node.IsValid() && Node.SimModule && Node.SimModule->IsEnabled())
			{
				if (FWheelBaseInterface* Wheel = Node.SimModule->Cast<FWheelBaseInterface>())
				{
					Wheel->ClearGroundBody();
				}
			}
		}
	}

	QUICK_SCOPE_CYCLE_COUNTER(STAT_VehicleSim_PerformAdditionalSimWork);
	if (SimModuleTree && RootParticle)
	{
		FRigidTransform3 ClusterWorldTM = FRigidTransform3(RootParticle->GetX(), RootParticle->GetR());
		AllInputs.VehicleWorldTransform = ClusterWorldTM;
		FVector ClusterVelocity = RootParticle->GetV();
		
		const TArray<FSimModuleTree::FSimModuleNode>& ModuleArray = SimModuleTree->GetSimulationModuleTree();
		
		
		//Collect all rays, find intersecting objects via aabb box of rays.
		FBox Box;
		Box.Init();
		for (const FSimModuleTree::FSimModuleNode& Node : ModuleArray)
		{
			if (Node.IsValid() && Node.SimModule && Node.SimModule->IsEnabled())
			{
				if (Node.SimModule->IsClustered() && Node.SimModule->IsBehaviourType(Chaos::eSimModuleTypeFlags::Raycast))
				{
					Chaos::FSpringTrace OutTrace;
					Chaos::FSuspensionBaseInterface* Suspension = static_cast<Chaos::FSuspensionBaseInterface*>(Node.SimModule);
					// would be cleaner an faster to just store radius in suspension also
					float WheelRadius = 0;
					if (Suspension->GetWheelSimTreeIndex() != Chaos::ISimulationModuleBase::INVALID_IDX)
					{
						Chaos::FWheelBaseInterface* Wheel = static_cast<Chaos::FWheelBaseInterface*>(ModuleArray[Suspension->GetWheelSimTreeIndex()].SimModule);
						if (Wheel)
						{
							WheelRadius = Wheel->GetWheelRadius();
						}
					}
					Suspension->GetWorldTraceEndpoints(DeltaSeconds, ClusterWorldTM, ClusterVelocity, WheelRadius, OutTrace);
					FVector TraceStart = OutTrace.Start;
					FVector TraceEnd = OutTrace.End;
					Box += TraceStart;
					Box += TraceEnd;
				}
			}
		}
		FCollisionQueryParams& TraceParams = InputData.PhysicsInputs.TraceParams;
		ECollisionChannel SpringCollisionChannel = InputData.PhysicsInputs.CollisionChannel;
		FCollisionResponseParams ResponseParams = InputData.PhysicsInputs.TraceCollisionResponse;
		TraceParams.bIgnoreTouches = true;
		TraceParams.bSkipNarrowPhase = true;
		TraceParams.bTraceComplex = true;
		
		TArray<FOverlapResult> Overlaps;
		bool bHasOverlap = false;
		if (InWorld && Box.IsValid && !Box.ContainsNaN() && Box.GetSize().Length() > 0)
		{
			FCollisionShape Shape = FCollisionShape::MakeBox(Box.GetExtent());
			bHasOverlap = FGenericPhysicsInterface::GeomOverlapMulti(
				InWorld,
				Shape,
				Box.GetCenter(),
				FQuat::Identity,
				Overlaps,
				SpringCollisionChannel,
				TraceParams,
				ResponseParams,
				FCollisionObjectQueryParams::DefaultObjectQueryParam);
		}

		TArray<FConstPhysicsObjectHandle> BlockingPhysicsObjects;
		TArray<FOverlapResult> BlockingOverlaps;
		if(bHasOverlap)
		{
			for(FOverlapResult& Overlap : Overlaps)
			{
				if (Overlap.PhysicsObject && Overlap.bBlockingHit)
				{
					if (TThreadParticle<EThreadContext::Internal>* Part = Overlap.PhysicsObject->GetParticle<EThreadContext::Internal>())
					{
						BlockingPhysicsObjects.Add(Overlap.PhysicsObject);
						BlockingOverlaps.Add(Overlap);;
					}
				}
			}
		}
		FReadPhysicsObjectInterface_Internal Interface = FPhysicsObjectInternalInterface::GetRead();
		FPhysicsObjectCollisionInterface_Internal CollisionInterface{ Interface };
		
		auto GetMaterialFromInternalFaceIndex_InternalHelper = [](const FPhysicsShape& Shape, const Chaos::FGeometryParticleHandle& PTActor, uint32 InternalFaceIndex)-> Chaos::FChaosPhysicsMaterial*
		{
			const auto& Materials = Shape.GetMaterials();
			if(Materials.Num() > 0 && PTActor.PhysicsProxy())
			{
				Chaos::FPBDRigidsSolver* Solver = PTActor.PhysicsProxy()->GetSolver<Chaos::FPBDRigidsSolver>();

				if(ensure(Solver))
				{
					if(Materials.Num() == 1)
					{
						return Solver->GetSimMaterials().Get(Materials[0].InnerHandle);
					}

					uint8 Index = Shape.GetGeometry()->GetMaterialIndex(InternalFaceIndex);

					if(Materials.IsValidIndex(Index))
					{
						return Solver->GetSimMaterials().Get(Materials[Index].InnerHandle);
					}
				}
			}

			return nullptr;
		};
		
		auto GetUserDataHelper = [](const Chaos::FChaosPhysicsMaterial& Material) -> UPhysicalMaterial*
		{
			void* UserData = Material.UserData;
			return UserData ? FChaosUserData::Get<UPhysicalMaterial>(UserData) : nullptr;
		};
		
		for (const FSimModuleTree::FSimModuleNode& Node : ModuleArray)
		{
			if (Node.IsValid() && Node.SimModule && Node.SimModule->IsEnabled())
			{
				if (Node.SimModule->IsClustered() && Node.SimModule->IsBehaviourType(Chaos::eSimModuleTypeFlags::Raycast))
				{
					QUICK_SCOPE_CYCLE_COUNTER(STAT_VehicleSim_SuspenssionRaycast);
					Chaos::FSpringTrace OutTrace;
					Chaos::FSuspensionBaseInterface* Suspension = static_cast<Chaos::FSuspensionBaseInterface*>(Node.SimModule);

					// would be cleaner an faster to just store radius in suspension also
					float WheelRadius = 0;
					if (Suspension->GetWheelSimTreeIndex() != Chaos::ISimulationModuleBase::INVALID_IDX)
					{
						Chaos::FWheelBaseInterface* Wheel = static_cast<Chaos::FWheelBaseInterface*>(ModuleArray[Suspension->GetWheelSimTreeIndex()].SimModule);
						if (Wheel)
						{
							WheelRadius = Wheel->GetWheelRadius();
						}
					}

					Suspension->GetWorldTraceEndpoints(DeltaSeconds, ClusterWorldTM, ClusterVelocity, WheelRadius, OutTrace);

					FVector TraceStart = OutTrace.Start;
					FVector TraceEnd = OutTrace.End;

					FVector TraceVector(TraceStart - TraceEnd);
					FVector TraceNormal = TraceVector.GetSafeNormal();
					
					ChaosInterface::FPTLocationHit HitResult = ChaosInterface::FPTLocationHit();
					bool bHasActualHit = false;
					if (InWorld && bHasOverlap && BlockingPhysicsObjects.Num() > 0)
					{
						switch (InputData.PhysicsInputs.TraceType)
						{
							case ETraceType::Spherecast:
								{
									QUICK_SCOPE_CYCLE_COUNTER(STAT_VehicleSim_ShapeSweep);
									ChaosInterface::FPTSweepHit SweepResult = ChaosInterface::FPTSweepHit();
									FSweepParameters SweepParams;
									SweepParams.bComputeMTD = false;
									SweepParams.bSweepComplex = TraceParams.bTraceComplex;
									FTransform SphereStart(FQuat::Identity,TraceStart);
									const FPhysicsShapeAdapter SphereShapeAdapter(FQuat::Identity, FCollisionShape::MakeSphere(WheelRadius));
									bHasActualHit = CollisionInterface.ShapeSweep(BlockingPhysicsObjects, SphereShapeAdapter.GetGeometry(), SphereStart, TraceEnd, SweepParams, SweepResult);
									if (bHasActualHit)
									{
										HitResult.WorldPosition = SweepResult.WorldPosition;
										HitResult.WorldNormal = SweepResult.WorldNormal;
										HitResult.Distance = SweepResult.Distance;
										HitResult.Actor = SweepResult.Actor;
										HitResult.Shape = SweepResult.Shape;
										HitResult.FaceIndex = SweepResult.FaceIndex;
										HitResult.ElementIndex = SweepResult.ElementIndex;
										HitResult.Flags	= SweepResult.Flags;
										HitResult.FaceNormal = SweepResult.FaceNormal;
									}
								}
							break;
							case ETraceType::Raycast:
							default:
								{
									ChaosInterface::FPTRaycastHit RayResult = ChaosInterface::FPTRaycastHit();
									QUICK_SCOPE_CYCLE_COUNTER(STAT_VehicleSim_LineTrace);
									bHasActualHit = CollisionInterface.LineTrace(BlockingPhysicsObjects, TraceStart, TraceEnd, TraceParams.bTraceComplex, RayResult);
									if (bHasActualHit)
									{
										HitResult.WorldPosition = RayResult.WorldPosition;
										HitResult.WorldNormal = RayResult.WorldNormal;
										HitResult.Distance = RayResult.Distance;
										HitResult.Actor = RayResult.Actor;
										HitResult.Shape = RayResult.Shape;
										HitResult.FaceIndex = RayResult.FaceIndex;
										HitResult.ElementIndex = RayResult.ElementIndex;
										HitResult.Flags	= RayResult.Flags;
										HitResult.FaceNormal = RayResult.FaceNormal;
									}
								}
							break;
						}
					}
					
					FOverlapResult ActualHitOverlapResult;
					FConstPhysicsObjectHandle ActualHitObjectHandle;
					if (bHasActualHit)
					{
						if (!HitResult.Actor)
						{
							bHasActualHit = false;
						}
						else
						{
							for (int32 Idx = 0; Idx < BlockingPhysicsObjects.Num(); ++Idx)
							{
							
								if (HitResult.Actor->PhysicsProxy() == BlockingPhysicsObjects[Idx]->PhysicsProxy())
								{
									ActualHitOverlapResult = BlockingOverlaps[Idx];
									ActualHitObjectHandle = BlockingPhysicsObjects[Idx];
									break;
								}
							}
							if (!ActualHitOverlapResult.bBlockingHit)
							{
								bHasActualHit = false;
							}
						}
					}

					float Offset = Suspension->GetMaxSpringLength();
					if (bHasActualHit && GModularVehicleDebugParams.SuspensionRaycastsEnabled)
					{
						Offset = HitResult.Distance - WheelRadius;

						if (Suspension->GetWheelSimTreeIndex() != Chaos::ISimulationModuleBase::INVALID_IDX)
						{
							const Chaos::FSimModuleTree::FSimModuleNode& WheelNode = ModuleArray[Suspension->GetWheelSimTreeIndex()];

							Chaos::FWheelBaseInterface* Wheel = static_cast<Chaos::FWheelBaseInterface*>(WheelNode.SimModule);
							if (Wheel)
							{
								if (GModularVehicleDebugParams.FrictionOverride > 0)
								{
									Wheel->SetSurfaceFriction(GModularVehicleDebugParams.FrictionOverride);
								}
								else
								{
									if (HitResult.Shape && HitResult.Actor && HitResult.Actor->PhysicsProxy() && HitResult.FaceIndex >= 0)
									{
										if (const FPhysicsMaterial* Material = GetMaterialFromInternalFaceIndex_InternalHelper(*HitResult.Shape, *HitResult.Actor, HitResult.FaceIndex))
										{
											Wheel->SetSurfaceFriction(Material->Friction);
										}
									}
								}

								
								IPhysicsProxyBase* HitProxy = nullptr;
								if (InWorld)
								{
									HitProxy = Chaos::FPhysicsObjectInterface::GetProxy({ &ActualHitObjectHandle, 1 });
									
									if (HitProxy && HitProxy->GetType() == EPhysicsProxyType::SingleParticleProxy)
									{
										if (FSingleParticlePhysicsProxy* ParticleProxy = static_cast<FSingleParticlePhysicsProxy*>(HitProxy))
										{
											Chaos::FPBDRigidParticleHandle* GroundParticle = HitResult.Actor->CastToRigidParticle(); 
											Wheel->SetGroundInteraction(GroundParticle, HitResult.WorldPosition, HitResult.WorldNormal);
										}
									}
								}
								 
							}
						}

#if CHAOS_DEBUG_DRAW
						if (GModularVehicleDebugParams.ShowSuspensionRaycasts)
						{
							Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(HitResult.WorldPosition, 3, 16, FColor::Red, false, -1.f, 0, 10.f);
						}

						if (Suspension->GetWheelSimTreeIndex() != Chaos::ISimulationModuleBase::INVALID_IDX)
						{
							Chaos::FWheelBaseInterface* Wheel = static_cast<Chaos::FWheelBaseInterface*>(ModuleArray[Suspension->GetWheelSimTreeIndex()].SimModule);
							if (Wheel)
							{
								if (GModularVehicleDebugParams.ShowWheelData)
								{
									FString TextOut = FString::Format(TEXT("{0}"), { Wheel->GetForceIntoSurface() });
									FColor Col = FColor::White;
									if (InWorld)
									{
										if (InWorld->GetNetMode() == ENetMode::NM_Client)
										{
											Col = FColor::Blue;
										}
										else
										{
											Col = FColor::Red;
										}
									}
									Chaos::FDebugDrawQueue::GetInstance().DrawDebugString(HitResult.WorldNormal + FVec3(0, 50, 50), TextOut, nullptr, Col, -1.f, true, 1.0f);
								}
							}
						}

#endif
					}

#if CHAOS_DEBUG_DRAW
					if (GModularVehicleDebugParams.ShowSuspensionRaycasts)
					{
						FColor DrawColor = FColor::Green;
						DrawColor = (bHasActualHit) ? FColor::Red : FColor::Green;
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugLine(TraceStart, TraceEnd, DrawColor, false, -1.f, 0, 2.f);
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(TraceStart, 3, 16, FColor::White, false, -1.f, 0, 10.f);
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugSphere(HitResult.WorldPosition, 1, 16, FColor::Red, false, -1.f, 0, 10.f);
						FString TextOut = FString::Format(TEXT("{0}"), { HitResult.Distance });

						FColor Col = FColor::White;
						if (InWorld)
						{
							if (InWorld->GetNetMode() == ENetMode::NM_Client)
							{
								Col = FColor::Blue;
							}
							else
							{
								Col = FColor::Red;
							}
						}
						Chaos::FDebugDrawQueue::GetInstance().DrawDebugString(HitResult.WorldPosition + FVector(0, 50, 50), TextOut, nullptr, Col, -1.f, true, 1.0f);
					}

					if (GModularVehicleDebugParams.ShowRaycastMaterial)
					{
						if (HitResult.Shape && HitResult.Actor && HitResult.Actor->PhysicsProxy() && HitResult.FaceIndex >= 0)
						{
							if (const FPhysicsMaterial* Material = GetMaterialFromInternalFaceIndex_InternalHelper(*HitResult.Shape, *HitResult.Actor, HitResult.FaceIndex))
							{
								if (UPhysicalMaterial* GTMaterial = GetUserDataHelper(*Material))
								{
									FDebugDrawQueue::GetInstance().DrawDebugString(HitResult.WorldPosition, GTMaterial->GetName(), nullptr, FColor::White, -1.f, true, 1.0f);
								}
							}
						}
					}

					if (GModularVehicleDebugParams.ShowWheelCollisionNormal)
					{
						FVector Pt = HitResult.WorldPosition;
						FDebugDrawQueue::GetInstance().DrawDebugLine(Pt, Pt + HitResult.WorldNormal * 20.0f, FColor::Yellow, false, 1.0f, 0, 1.0f);
						FDebugDrawQueue::GetInstance().DrawDebugSphere(Pt, 5.0f, 4, FColor::White, false, 1.0f, 0, 1.0f);
					}

#endif
					Suspension->SetSpringLength(Offset, WheelRadius);
					FVector Up = ClusterWorldTM.GetUnitAxis(EAxis::Z);

					FVector HitPoint;
					float HitDistance = Offset;
					if (InputData.PhysicsInputs.TraceType == ETraceType::Spherecast)
					{
						HitPoint = HitResult.WorldPosition;
					}
					else
					{
						HitPoint = HitResult.WorldPosition + Up * WheelRadius;
					}
					
					TEnumAsByte<EPhysicalSurface> SurfaceType = EPhysicalSurface::SurfaceType_Default;
					if (HitResult.Shape && HitResult.Actor && HitResult.FaceIndex >= 0)
					{
						if (const FPhysicsMaterial* Material = GetMaterialFromInternalFaceIndex_InternalHelper(*HitResult.Shape, *HitResult.Actor, HitResult.FaceIndex))
						{
							if (UPhysicalMaterial* GTMaterial = GetUserDataHelper(*Material))
							{
								SurfaceType = GTMaterial->SurfaceType;
							}
						}
					}
					
					FSuspensionTargetPoint TargetPoint(
						HitPoint
						, HitResult.WorldNormal
						, HitDistance
						, bHasActualHit
						, SurfaceType
						, bHasActualHit && HitResult.Actor ? HitResult.Actor->PhysicsProxy() : nullptr
					);

					Suspension->SetTargetPoint(TargetPoint);
				}

			}

		}

	}
}

void FModularVehicleSimulation::ApplyDeferredForces(IPhysicsProxyBase* Proxy)
{
	using namespace Chaos;

	Chaos::EnsureIsInPhysicsThreadContext();

	if (SimModuleTree && Proxy)
	{

		SimModuleTree->AccessDeferredForces().Apply(RootParticle);
		
	}
}

void FModularVehicleSimulation::FillOutputState(FModularVehicleAsyncOutput& Output)
{
	QUICK_SCOPE_CYCLE_COUNTER(Stat_FModularVehicleSimulation_FillOutputState);

	Output.VehicleSimOutput.NewlyCreatedModuleGuids = NewlyCreatedModuleGuids;
	NewlyCreatedModuleGuids.Empty();

	if (Chaos::FSimModuleTree* SimTree = GetSimComponentTree().Get())
	{
		for (int I = 0; I < SimTree->GetNumNodes(); I++)
		{
			if (SimTree->GetSimModule(I))
			{
				if (Chaos::FSimOutputData* OutData = SimTree->AccessSimModule(I)->GenerateOutputData())
				{
					OutData->FillOutputState(SimTree->GetSimModule(I));
					Output.VehicleSimOutput.SimTreeOutputData.Add(OutData);
				}
			}
		}
	}
}


void FModularVehicleSimulation::AppendTreeUpdates(Chaos::FSimTreeUpdates* InNextTreeUpdatesInternal)
{
	Chaos::EnsureIsInGameThreadContext();

	if (InNextTreeUpdatesInternal == nullptr)
	{
		return;
	}

	UE::TWriteScopeLock InputConfigLock(TreeConfigurationLock);
	NextTreeUpdatesInternal.Add(*InNextTreeUpdatesInternal);
}

void FModularVehicleSimulation::ActionTreeUpdates()
{
	Chaos::EnsureIsInPhysicsThreadContext();

	if (NextTreeUpdatesInternal.IsEmpty())
	{
		return;
	}

	UE::TReadScopeLock InputConfigLock(TreeConfigurationLock);

	if (SimModuleTree.IsValid())
	{
		for (Chaos::FSimTreeUpdates& TreeUpdate : NextTreeUpdatesInternal)
		{
			SimModuleTree->AppendTreeUpdates(TreeUpdate);
			FModularVehicleBuilder::FixupTreeLinks(SimModuleTree);

			// NewlyCreatedModuleGuids will be passed back to GT to inform that these now exist
			for (const Chaos::FPendingModuleAdds& ModuleAdd : TreeUpdate.GetNewModules())
			{
				if (ModuleAdd.NewSimModule)
				{
					NewlyCreatedModuleGuids.Add(Chaos::FCreatedModule(
						ModuleAdd.NewSimModule->GetSimType(),
						ModuleAdd.NewSimModule->GetGuid(),
						ModuleAdd.NewSimModule->GetTreeIndex()));
				}
			}
		}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
		if (bModularVehicle_DumpModuleTreeStructure_Enabled)
		{
			UE_LOGF(LogModularVehicleSim, Warning, "SimTreeModules:");
			for (int I = 0; I < SimModuleTree->GetNumNodes(); I++)
			{
				if (Chaos::ISimulationModuleBase* Module = SimModuleTree->GetNode(I).SimModule)
				{
					FString String;
					Module->GetDebugString(String);
					UE_LOGF(LogModularVehicleSim, Warning, "..%ls", *String);
				}
			}
		}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)

	}

	NextTreeUpdatesInternal.Empty();
}

void FModularVehicleSimulation::GenerateReplicationStructure(FNetworkModularVehicleStates& State)
{
	Chaos::EnsureIsInGameThreadContext();

	// ensure tree resizing from ActionTreeUpdates doesn't run at the same time as this
	UE::TReadScopeLock InputConfigLock(TreeConfigurationLock);

	State.ModuleData.Empty();
	if (SimModuleTree.IsValid())
	{
		SimModuleTree->GenerateReplicationStructure(State.ModuleData);
	}
}

