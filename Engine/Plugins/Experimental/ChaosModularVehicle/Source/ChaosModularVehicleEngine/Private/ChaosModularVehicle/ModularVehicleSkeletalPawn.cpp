// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ModularVehicleSkeletalPawn.h"
#include "Components/SkeletalMeshComponent.h"
#include "ChaosModularVehicle/ModularVehicleBaseComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularVehicleSkeletalPawn)


AModularVehicleSkeletalPawn::AModularVehicleSkeletalPawn(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	SkeletalMeshComponent = CreateDefaultSubobject<USkeletalMeshComponent>(TEXT("SkeletalMeshComponent0"));
	SetRootComponent(SkeletalMeshComponent);

	VehicleSimComponent = CreateDefaultSubobject<UModularVehicleBaseComponent>(TEXT("VehicleSimComponent0"));
	VehicleSimComponent->SetClusterComponent(nullptr); // no cluster union component in this scenario

	SetRemoteRoleForBackwardsCompat(ROLE_SimulatedProxy);
	bReplicates = true;
	SetReplicatingMovement(true);
}
