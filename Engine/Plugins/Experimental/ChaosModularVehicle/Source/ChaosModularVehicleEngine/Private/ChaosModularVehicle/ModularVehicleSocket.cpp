// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/ModularVehicleSocket.h"
#include "ChaosModularVehicle/ClusterUnionVehicleComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ModularVehicleSocket)

FModularVehicleSocket::FModularVehicleSocket()
{
	RelativeLocation = FVector::ZeroVector;
	RelativeRotation = FRotator::ZeroRotator;
}

FVector FModularVehicleSocket::GetLocation(const class UClusterUnionVehicleComponent* Component) const
{
	if (ensure(Component))
	{
		const FTransform& LocalT = GetLocalTransform();
		FTransform ComponentT = Component->GetComponentTransform();
		return (LocalT * ComponentT).GetLocation();
	}
	return FVector(0.f);
}

FTransform FModularVehicleSocket::GetLocalTransform() const
{
	return FTransform(RelativeRotation, RelativeLocation);
}

FTransform FModularVehicleSocket::GetTransform(const class UClusterUnionVehicleComponent* Component) const
{
	if (Component)
	{
		const FTransform& LocalT = GetLocalTransform();
		FTransform ComponentT = Component->GetComponentTransform();
		return (LocalT * ComponentT);
	}
	return FTransform::Identity;
}

