// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/SuspensionSimModule.h"
#include "SimModule/SimModuleTree.h"
#include "SimModule/TorqueSimModule.h"
#include "Chaos/PBDSuspensionConstraints.h"
#include "PhysicsProxy/SuspensionConstraintProxy.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"
#include "PBDRigidsSolver.h"
#include "VehicleUtility.h"
#include "PhysicsEngine/PhysicsObjectExternalInterface.h"
#include "Physics/PhysicsInterfaceCore.h"


#if VEHICLE_DEBUGGING_ENABLED
UE_DISABLE_OPTIMIZATION_SHIP
#endif

using namespace Chaos;

	FSuspensionSimModule::FSuspensionSimModule(const FSuspensionSettings& Settings)
		: TSimModuleSettings<FSuspensionSettings>(Settings)
		, SpringDisplacement(0.f)
		, LastDisplacement(0.f)
		, SpringSpeed(0.f)
	{
		AccessSetup().MaxLength = FMath::Abs(Settings.MaxRaise + Settings.MaxDrop);
	}
	FSuspensionSimModule::~FSuspensionSimModule()
	{
	}

	float FSuspensionSimModule::GetSpringLength() const
	{
		return  -(Setup().MaxLength - SpringDisplacement);
	}

	void FSuspensionSimModule::SetSpringLength(float InLength, float WheelRadius)
	{
		float DisplacementInput = InLength;
		DisplacementInput = FMath::Max(0.f, DisplacementInput);
		SpringDisplacement = Setup().MaxLength - DisplacementInput;
	}

	void FSuspensionSimModule::GetWorldTraceEndpoints(float DeltaSeconds, const FTransform& BodyTransform, const FVector& Velocity, float WheelRadius, FSpringTrace& OutTrace) const
	{
		FVector LocalDirection = Setup().SuspensionAxis;
		FVector Local = GetParentRelativeTransform().GetLocation(); // change to just a vector and GetLocalLocation
		FVector WorldLocation = BodyTransform.TransformPosition(Local);
		FVector WorldDirection = BodyTransform.TransformVector(LocalDirection);

		FVector MovementExpansion = FVector::ZeroVector;
		// time dilation only an issue when time is slowed down
		if (CurrentTimeDilation < 1.0f)
		{
			float Amount = FMath::Abs(WorldDirection.Dot(Velocity));
			MovementExpansion += WorldDirection * (Amount * DeltaSeconds / CurrentTimeDilation);
		}

		OutTrace.Start = WorldLocation - WorldDirection * (Setup().MaxRaise);
		OutTrace.End = WorldLocation + WorldDirection * (Setup().MaxDrop + WheelRadius) + MovementExpansion;
	}

	void FSuspensionSimModule::OnConstruction_External(const Chaos::FPhysicsObjectHandle& PhysicsObject)
	{
		Chaos::EnsureIsInGameThreadContext();
		CreateConstraint(PhysicsObject);
	}

	void FSuspensionSimModule::OnTermination_External()
	{
		Chaos::EnsureIsInGameThreadContext();
		DestroyConstraint();
	}

	void FSuspensionSimModule::Simulate(float DeltaTime, const FAllInputs& Inputs, FSimModuleTree& VehicleModuleSystem)
	{
		{
			CurrentTimeDilation = FMath::Max(Inputs.CurrentTimeDilation, SMALL_NUMBER);
			float ForceIntoSurface = 0.0f;
			if (SpringDisplacement > 0)
			{
				float Damping = Setup().SpringDamping;
				DeltaTime = FMath::Max(DeltaTime, SMALL_NUMBER);
				SpringSpeed = (LastDisplacement - SpringDisplacement) / DeltaTime;

				float StiffnessForce = SpringDisplacement * Setup().SpringRate;
				float DampingForce = SpringSpeed * Damping;
				float SuspensionForce = StiffnessForce - DampingForce;
				LastDisplacement = SpringDisplacement;

				if (SuspensionForce > 0)
				{
					ForceIntoSurface = SuspensionForce * Setup().SuspensionForceEffect;

					if (!ConstraintHandle.IsValid())
					{
						AddLocalForce(Setup().SuspensionAxis * -SuspensionForce, true, false, true, FColor::Green);
					}
				}
			}

			// tell wheels how much they are being pressed into the ground
			if (SimModuleTree && WheelSimTreeIndex != INVALID_IDX)
			{
				if (Chaos::ISimulationModuleBase* Module = SimModuleTree->AccessSimModule(WheelSimTreeIndex))
				{
					if(Chaos::FWheelBaseInterface* Wheel = Module->Cast<Chaos::FWheelBaseInterface>())
					{
						Wheel->SetForceIntoSurface(ForceIntoSurface);
					}
				}
		
			}
		}

		if (ConstraintHandle.IsValid())
		{
			UpdateConstraint();
		}

	}

	void FSuspensionSimModule::Animate()
	{
		FVector Movement = -Setup().SuspensionAxis * (Setup().MaxRaise + GetSpringLength());

		AnimationData.AnimFlags = EAnimationFlags::AnimatePosition;
		AnimationData.AnimationLocOffset = Movement;
	}


	void FSuspensionSimModule::CreateConstraint(const Chaos::FPhysicsObjectHandle& PhysicsObject)
	{
		Chaos::EnsureIsInGameThreadContext();

		const FVector& LocalOffset = GetInitialParticleTransform().GetLocation();

		if (FChaosScene* Scene = static_cast<FChaosScene*>(FPhysicsObjectExternalInterface::GetScene({ &PhysicsObject, 1 })))
		{
			FLockedWritePhysicsObjectExternalInterface Interface = FPhysicsObjectExternalInterface::LockWrite(Scene);
			if (const FGeometryParticle* Particle = Interface->GetParticle(PhysicsObject))
			{
				ConstraintHandle = FPhysicsInterface::CreateSuspension(PhysicsObject, LocalOffset);

				if (ConstraintHandle.IsValid())
				{
					if (FSuspensionConstraint* Constraint = static_cast<FSuspensionConstraint*>(ConstraintHandle.Constraint))
					{
						Constraint->SetHardstopStiffness(1.0f);
						Constraint->SetSpringStiffness(Setup().SpringRate * 0.25f);
						Constraint->SetSpringPreload(Setup().SpringPreload);
						Constraint->SetSpringDamping(Setup().SpringDamping * 5.0f);
						Constraint->SetMinLength(-Setup().MaxRaise);
						Constraint->SetMaxLength(Setup().MaxDrop);
						Constraint->SetAxis(-Setup().SuspensionAxis);
					}
				}
			}
		}
	}

	void FSuspensionSimModule::DestroyConstraint()
	{
		Chaos::EnsureIsInGameThreadContext();
		FPhysicsCommand::ExecuteWrite(ConstraintHandle, [&](const FPhysicsConstraintHandle& Constraint)
			{
				FPhysicsInterface::ReleaseConstraint(ConstraintHandle);
			});
	}

	void FSuspensionSimModule::UpdateConstraint()
	{
		if (Chaos::FSuspensionConstraint* Constraint = static_cast<Chaos::FSuspensionConstraint*>(ConstraintHandle.Constraint))
		{
			if (Constraint && Constraint->IsValid())
			{
				if (FSuspensionConstraintPhysicsProxy* Proxy = Constraint->GetProxy<FSuspensionConstraintPhysicsProxy>())
				{
					Chaos::FPhysicsSolver* Solver = Proxy->GetSolver<Chaos::FPhysicsSolver>();
					const FVector& CurrentTargetPosition = GetTargetPosition();
					const FVector& CurrentImpactNormal = GetImpactNormal();
					const IPhysicsProxyBase* GroundProxy = GetHitProxy();
					const bool bCurrentWheelInContact = IsWheelInContact();
					Solver->SetSuspensionTarget(Constraint, CurrentTargetPosition, CurrentImpactNormal, bCurrentWheelInContact, GroundProxy);
				}
			}
		}
	}

	void FSuspensionSimModuleData::FillSimState(ISimulationModuleBase* SimModule)
	{
		if (FSuspensionSimModule* Sim = SimModule->Cast<FSuspensionSimModule>())
		{
			Sim->SpringDisplacement = SpringDisplacement;
			Sim->LastDisplacement = LastDisplacement;
		}
	}

	void FSuspensionSimModuleData::FillNetState(const ISimulationModuleBase* SimModule)
	{
		if (const FSuspensionSimModule* Sim = SimModule->Cast<const FSuspensionSimModule>())
		{
			SpringDisplacement = Sim->SpringDisplacement;
			LastDisplacement = Sim->LastDisplacement;
		}
	}

	void FSuspensionSimModuleData::Lerp(const float LerpFactor, const FModuleNetData& Min, const FModuleNetData& Max)
	{
		const FSuspensionSimModuleData& MinData = static_cast<const FSuspensionSimModuleData&>(Min);
		const FSuspensionSimModuleData& MaxData = static_cast<const FSuspensionSimModuleData&>(Max);

		SpringDisplacement = FMath::Lerp(MinData.SpringDisplacement, MaxData.SpringDisplacement, LerpFactor);
		LastDisplacement = FMath::Lerp(MinData.LastDisplacement, MaxData.LastDisplacement, LerpFactor);
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FString FSuspensionSimModuleData::ToString() const
	{
		return FString::Printf(TEXT("Module:%s SpringDisplacement:%f LastDisplacement:%f"),
			*DebugString, SpringDisplacement, LastDisplacement);
	}
#endif

	void FSuspensionOutputData::FillOutputState(const ISimulationModuleBase* SimModule)
	{
		FSimOutputData::FillOutputState(SimModule);

		if (const FSuspensionSimModule* Sim = SimModule->Cast<const FSuspensionSimModule>())
		{
			SpringDisplacement = Sim->SpringDisplacement;
			SpringDisplacementVector = -Sim->Setup().SuspensionAxis * Sim->SpringDisplacement + Sim->GetAnimationOffset();
			SpringSpeed = Sim->SpringSpeed;
			ImpactNormal = Sim->GetImpactNormal();

			// Required to recalculate suspension length on the game thread (post physics integration) so it looks correct visually
			ImpactPosition = Sim->GetTargetPosition();
			ModuleLocalPosition = Sim->GetParentRelativeTransform().GetLocation();
			LocalSuspensionAxis = Sim->Setup().SuspensionAxis;
			MaxRaise = Sim->Setup().MaxRaise;
			MaxDrop = Sim->Setup().MaxDrop;
			bWheelHit = Sim->IsWheelInContact();
		}
	}

	void FSuspensionOutputData::Lerp(const FSimOutputData& InCurrent, const FSimOutputData& InNext, float Alpha)
	{
		FSimOutputData::Lerp(InCurrent, InNext, Alpha);

		const FSuspensionOutputData& Current = static_cast<const FSuspensionOutputData&>(InCurrent);
		const FSuspensionOutputData& Next = static_cast<const FSuspensionOutputData&>(InNext);

		SpringDisplacement = FMath::Lerp(Current.SpringDisplacement, Next.SpringDisplacement, Alpha);
		SpringSpeed = FMath::Lerp(Current.SpringSpeed, Next.SpringSpeed, Alpha);
		ImpactNormal = FMath::Lerp(Current.ImpactNormal, Next.ImpactNormal, Alpha);

		ImpactPosition = Next.ImpactPosition;
		ModuleLocalPosition = FMath::Lerp(Current.ModuleLocalPosition, Next.ModuleLocalPosition, Alpha);
		LocalSuspensionAxis = FMath::Lerp(Current.LocalSuspensionAxis, Next.LocalSuspensionAxis, Alpha);
		MaxRaise = FMath::Lerp(Current.MaxRaise, Next.MaxRaise, Alpha);
		MaxDrop = FMath::Lerp(Current.MaxDrop, Next.MaxDrop, Alpha);
		bWheelHit = Next.bWheelHit;

	}


	void FSuspensionOutputData::GetFinalAnimDataGameThread(const FTransform& NewTransform, Chaos::FSimModuleAnimationData& AnimDataOut)
	{
		FSimOutputData::GetFinalAnimDataGameThread(NewTransform, AnimDataOut);

		FVector WorldLocation = NewTransform.TransformPosition(ModuleLocalPosition);
		FVector WorldDirection = NewTransform.TransformVector(LocalSuspensionAxis);
		FVector NewStart = WorldLocation - WorldDirection * (MaxRaise);
		FVector NewEnd = ImpactPosition;
		float Length = FVector::DotProduct(NewStart - NewEnd, WorldDirection);
		Length = FMath::Clamp(Length, -(MaxDrop+MaxRaise), 0);
		FVector Movement = -LocalSuspensionAxis * (Length + MaxRaise);

		if (!bWheelHit)
		{
			Movement = LocalSuspensionAxis * MaxDrop;
		}

		AnimDataOut.AnimFlags = EAnimationFlags::AnimatePosition;
		AnimDataOut.AnimationLocOffset = Movement;
	}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FString FSuspensionOutputData::ToString()
	{
		return FString::Printf(TEXT("%s, SpringDisplacement=%3.3f, SpringSpeed=%3.3f"), *DebugString, SpringDisplacement, SpringSpeed);
	}
#endif


#if VEHICLE_DEBUGGING_ENABLED
UE_ENABLE_OPTIMIZATION_SHIP
#endif
