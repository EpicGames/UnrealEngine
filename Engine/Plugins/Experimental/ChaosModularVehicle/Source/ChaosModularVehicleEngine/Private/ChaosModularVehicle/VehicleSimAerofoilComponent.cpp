// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimAerofoilComponent.h"
#include "SimModule/SimModulesInclude.h"
#include "VehicleUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimAerofoilComponent)


UVehicleSimAerofoilComponent::UVehicleSimAerofoilComponent()
{
	// set defaults
	Offset = FVector::ZeroVector;
	ForceAxis = FVector(0, 0, 1);
	ControlRotationAxis = FVector(0, 1, 0);
	Area = 10.0f;
	Camber = 10.0f;
	MaxControlAngle = 30.0f;
	StallAngle = 20.0f;
	Type = EModuleAerofoilType::Wing;
	LiftMultiplier = 1.0f;
	DragMultiplier = 1.0f;
	AnimationMagnitudeMultiplier = 1.0f;
	bAnimationEnabled = true;
}

Chaos::ISimulationModuleBase* UVehicleSimAerofoilComponent::CreateNewCoreModule() const
{
	Chaos::FAerofoilSettings Settings;

	Settings.Offset = Offset;
	Settings.ForceAxis = ForceAxis;
	Settings.ControlRotationAxis = ControlRotationAxis;
	Settings.Area = Area;
	Settings.Camber = Camber;
	Settings.MaxControlAngle = MaxControlAngle;
	Settings.StallAngle = StallAngle;
	Settings.Type = static_cast<Chaos::EAerofoil>(Type);
	Settings.LiftMultiplier = LiftMultiplier;
	Settings.DragMultiplier = DragMultiplier;
	Settings.AnimationMagnitudeMultiplier = AnimationMagnitudeMultiplier;

	Chaos::ISimulationModuleBase* Aerofoil = new Chaos::FAerofoilSimModule(Settings);
	Aerofoil->SetAnimationEnabled(bAnimationEnabled);

	return Aerofoil;
}
