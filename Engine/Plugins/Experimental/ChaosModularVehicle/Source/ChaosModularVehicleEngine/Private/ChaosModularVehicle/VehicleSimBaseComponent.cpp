// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimBaseComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimBaseComponent)

void UVehicleSimBaseComponent::SetTreeIndex(const int32 NewValue)
{
	TreeIndex = NewValue;
}
