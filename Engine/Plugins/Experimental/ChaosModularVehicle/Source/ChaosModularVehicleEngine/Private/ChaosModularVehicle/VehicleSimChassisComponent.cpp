// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimChassisComponent.h"
#include "SimModule/SimModulesInclude.h"
#include "VehicleUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimChassisComponent)

UVehicleSimChassisComponent::UVehicleSimChassisComponent()
{
	AreaMetresSquared = 0.0f;
	DragCoefficient = 0.5f;
	DensityOfMedium = Chaos::RealWorldConsts::AirDensity();
	XAxisMultiplier = 1.0f;
	YAxisMultiplier = 1.0f;
	AngularDamping = 0.0f;
}

Chaos::ISimulationModuleBase* UVehicleSimChassisComponent::CreateNewCoreModule() const
{
	Chaos::FChassisSettings Settings;
	Settings.AreaMetresSquared = AreaMetresSquared;
	Settings.DragCoefficient = DragCoefficient;
	Settings.DensityOfMedium = DensityOfMedium;
	Settings.XAxisMultiplier = XAxisMultiplier;
	Settings.YAxisMultiplier = YAxisMultiplier;
	Settings.AngularDamping = AngularDamping;

	Chaos::ISimulationModuleBase* Chassis = new Chaos::FChassisSimModule(Settings);

	return Chassis;
}
