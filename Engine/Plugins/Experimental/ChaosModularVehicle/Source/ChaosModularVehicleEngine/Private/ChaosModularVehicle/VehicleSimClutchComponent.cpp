// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimClutchComponent.h"
#include "SimModule/SimModulesInclude.h"
#include "VehicleUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimClutchComponent)

UVehicleSimClutchComponent::UVehicleSimClutchComponent()
{
	// set defaults
	ClutchStrength = 1.0f;
}

Chaos::ISimulationModuleBase* UVehicleSimClutchComponent::CreateNewCoreModule() const
{
	// use the UE properties to setup the physics state
	Chaos::FClutchSettings Settings;

	Settings.ClutchStrength = ClutchStrength;
	Chaos::ISimulationModuleBase* Clutch = new Chaos::FClutchSimModule(Settings);
	return Clutch;
}

