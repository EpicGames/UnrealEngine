// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimEngineComponent.h"
#include "SimModule/SimModulesInclude.h"
#include "VehicleUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimEngineComponent)

UVehicleSimEngineComponent::UVehicleSimEngineComponent()
{
	// set defaults
	// TODO
	TorqueCurve.GetRichCurve()->AddKey(0, 0.5f);
	TorqueCurve.GetRichCurve()->AddKey(0.5, 1.0f);
	TorqueCurve.GetRichCurve()->AddKey(1.0f, 0.75f);
	MaxTorque = 200;
	MaxRPM = 5000;
	EngineIdleRPM = 1200;
	EngineBrakeEffect = 150.0f;
	EngineInertia = 1000.0f; 	// [Kg.m-2] How hard it is to turn the engine
}

Chaos::ISimulationModuleBase* UVehicleSimEngineComponent::CreateNewCoreModule() const
{
	// use the UE properties to setup the physics state
	Chaos::FEngineSettings Settings;

	Settings.MaxTorque = Chaos::TorqueMToCm(MaxTorque);

	float NumSamples = 20;
	for (float X = 0; X <= MaxRPM; X += (MaxRPM / NumSamples))
	{
		float MinVal = 0.f, MaxVal = 0.f;
		TorqueCurve.GetRichCurveConst()->GetValueRange(MinVal, MaxVal);
		float Y = this->TorqueCurve.GetRichCurveConst()->Eval(X) / MaxVal;
		Settings.TorqueCurve.AddNormalized(Y);
	}

	Settings.MaxRPM = MaxRPM;
	Settings.IdleRPM = EngineIdleRPM;
	Settings.EngineBrakeEffect = EngineBrakeEffect;
	Settings.EngineInertia = EngineInertia;

	Chaos::ISimulationModuleBase* Engine = new Chaos::FEngineSimModule(Settings);
	return Engine;
}

void UVehicleSimEngineComponent::OnOutputReady(const Chaos::FSimOutputData* OutputData)
{

}
