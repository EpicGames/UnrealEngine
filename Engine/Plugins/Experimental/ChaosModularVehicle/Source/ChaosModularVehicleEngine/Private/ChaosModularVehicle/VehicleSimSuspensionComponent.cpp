// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimSuspensionComponent.h"
#include "SimModule/SimModulesInclude.h"
#include "ChaosModularVehicle/SuspensionSimModule.h"
#include "VehicleUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimSuspensionComponent)


UVehicleSimSuspensionComponent::UVehicleSimSuspensionComponent()
{
	// set defaults
	SuspensionAxis = FVector(0, 0, -1);
	SuspensionMaxRaise = 5.0f;
	SuspensionMaxDrop = 5.0f;
	SpringRate = 100.0f;
	SpringPreload = 50.0f;
	SpringDamping = 0.9f;
	SuspensionForceEffect = 100.0f;
	bAnimationEnabled = true;
}

Chaos::ISimulationModuleBase* UVehicleSimSuspensionComponent::CreateNewCoreModule() const
{
	// use the UE properties to setup the physics state
	FSuspensionSettings Settings;

	Settings.SuspensionAxis = SuspensionAxis;
	Settings.MaxRaise = SuspensionMaxRaise;
	Settings.MaxDrop = SuspensionMaxDrop;
	Settings.SpringRate = Chaos::MToCm(SpringRate);
	Settings.SpringPreload = Chaos::MToCm(SpringPreload);
	Settings.SpringDamping = SpringDamping;
	Settings.SuspensionForceEffect = SuspensionForceEffect;

	//Settings.SwaybarEffect = SwaybarEffect;

	Chaos::ISimulationModuleBase* Suspension = new FSuspensionSimModule(Settings);
	Suspension->SetAnimationEnabled(bAnimationEnabled);
	return Suspension;

};
