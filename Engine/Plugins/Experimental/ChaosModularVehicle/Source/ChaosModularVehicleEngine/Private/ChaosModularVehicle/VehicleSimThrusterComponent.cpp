// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimThrusterComponent.h"
#include "SimModule/SimModulesInclude.h"
#include "VehicleUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimThrusterComponent)

UVehicleSimThrusterComponent::UVehicleSimThrusterComponent()
{
	MaxThrustForce = 10000.0f;
	ForceAxis = FVector(1.0f, 0.0f, 0.0f);
	ForceOffset = FVector::ZeroVector;
	bSteeringEnabled = false;
	SteeringAxis = FVector(0.0f, 0.0f, 1.0f);
	MaxSteeringAngle = 0.0f;
	SteeringForceEffect = 0.5f;
	BoostMultiplierEffect = 2.0f;
	bAnimationEnabled = true;
}

Chaos::ISimulationModuleBase* UVehicleSimThrusterComponent::CreateNewCoreModule() const
{
	Chaos::FThrusterSettings Settings;

	Settings.MaxThrustForce = MaxThrustForce;
	Settings.ForceAxis = ForceAxis;
	Settings.ForceOffset = ForceOffset;
	Settings.SteeringEnabled = bSteeringEnabled;
	Settings.SteeringAxis = SteeringAxis;
	Settings.MaxSteeringAngle = MaxSteeringAngle;
	Settings.SteeringForceEffect = SteeringForceEffect;
	Settings.BoostMultiplier = BoostMultiplierEffect;

	Chaos::ISimulationModuleBase* Thruster = new Chaos::FThrusterSimModule(Settings);
	Thruster->SetAnimationEnabled(bAnimationEnabled);

	return Thruster;
}
