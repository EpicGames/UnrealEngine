// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimTransmissionComponent.h"
#include "SimModule/SimModulesInclude.h"
#include "VehicleUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimTransmissionComponent)


UVehicleSimTransmissionComponent::UVehicleSimTransmissionComponent()
{
	// set defaults
	ForwardRatios.Add(2.85f);
	ForwardRatios.Add(2.02f);
	ForwardRatios.Add(1.35f);
	ForwardRatios.Add(1.0f);

	ReverseRatios.Add(2.86f);

	FinalDriveRatio = 3.08f;

	ChangeUpRPM = 4500;
	ChangeDownRPM = 1600;
	GearChangeTime = 0.5f;
	GearHysteresisTime = 2.0f;
	TransmissionEfficiency = 0.9f;
	TransmissionType = EModuleTransType::Automatic;
	AutoReverse = true;
}

Chaos::ISimulationModuleBase* UVehicleSimTransmissionComponent::CreateNewCoreModule() const
{
	// use the UE properties to setup the physics state
	Chaos::FTransmissionSettings Settings;

	Settings.ForwardRatios = ForwardRatios;
	Settings.ReverseRatios = ReverseRatios;
	Settings.FinalDriveRatio = FinalDriveRatio;
	Settings.ChangeUpRPM = ChangeUpRPM;
	Settings.ChangeDownRPM = ChangeDownRPM;
	Settings.GearChangeTime = GearChangeTime;
	Settings.GearHysteresisTime = GearHysteresisTime;
	Settings.TransmissionEfficiency = TransmissionEfficiency;
	Settings.TransmissionType = static_cast<Chaos::FTransmissionSettings::ETransType>(TransmissionType);

	Settings.AutoReverse = AutoReverse;

	Chaos::ISimulationModuleBase* Transmission = new Chaos::FTransmissionSimModule(Settings);
	return Transmission;
}

void UVehicleSimTransmissionComponent::OnOutputReady(const Chaos::FSimOutputData* OutputData)
{
	if (OutputData)
	{
		const Chaos::FTransmissionOutputData* TransOutData = static_cast<const Chaos::FTransmissionOutputData*>(OutputData);

		for (const Chaos::FGearChangeEvent& Event : TransOutData->GearChangeEvents)
		{
			OnGearChangeNativeEvent.Broadcast(TransOutData->ModuleGuid, Event.ChangedToGear);
		}
	}
}
