// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosModularVehicle/VehicleSimWheelComponent.h"
#include "SimModule/SimModulesInclude.h"
#include "VehicleUtility.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(VehicleSimWheelComponent)


UVehicleSimWheelComponent::UVehicleSimWheelComponent()
{
	// set defaults
	WheelRadius = 30.0f;
	WheelWidth = 20.0f;
	WheelInertia = 10.0f;
	FrictionMultiplier = 2.0f;
	CorneringStiffness = 1000.0f;
	SlipAngleLimit = 8.0f;

	MaxBrakeTorque = 2000.0f;
	bHandbrakeEnabled = false;
	HandbrakeTorque = 2000.0f;
	bSteeringEnabled = false;
	MaxSteeringAngle = 35;

	bABSEnabled = true;
	bTractionControlEnabled = true;
	AxisType = EWheelAxisType::Y;
	ReverseDirection = false;
	ForceOffset = FVector::ZeroVector;
	bAnimationEnabled = true;
}

Chaos::ISimulationModuleBase* UVehicleSimWheelComponent::CreateNewCoreModule() const
{
	// use the UE properties to setup the physics state
	Chaos::FWheelSettings Settings;
	Settings.Radius = WheelRadius;
	Settings.Width = WheelWidth;
	Settings.WheelInertia = WheelInertia;
	Settings.FrictionMultiplier = FrictionMultiplier;
	Settings.CorneringStiffness = CorneringStiffness * 10000.0f;
	Settings.SlipAngleLimit = SlipAngleLimit;

	Settings.MaxBrakeTorque = Chaos::TorqueMToCm(MaxBrakeTorque);
	Settings.HandbrakeEnabled = bHandbrakeEnabled;
	Settings.HandbrakeTorque = Chaos::TorqueMToCm(HandbrakeTorque);
	Settings.SteeringEnabled = bSteeringEnabled;
	Settings.MaxSteeringAngle = bSteeringEnabled ? MaxSteeringAngle : 0.0f;
	Settings.ABSEnabled = bABSEnabled;
	Settings.TractionControlEnabled = bTractionControlEnabled;
	Settings.Axis = (Chaos::EWheelAxis)(AxisType);
	Settings.ReverseDirection = ReverseDirection;
	Settings.ForceOffset = ForceOffset;

	Chaos::ISimulationModuleBase* Wheel = new Chaos::FWheelSimModule(Settings);
	Wheel->SetAnimationEnabled(bAnimationEnabled);

	return Wheel;
}

void UVehicleSimWheelComponent::OnOutputReady(const Chaos::FSimOutputData* OutputData)
{
	if (OutputData)
	{
		const Chaos::FWheelOutputData* WheelOutData = static_cast<const Chaos::FWheelOutputData*>(OutputData);

		for (const Chaos::FWheelTouchChangeEvent& Event : WheelOutData->WheelTouchEvents)
		{
			OnWheelTouchChangeNativeEvent.Broadcast(WheelOutData->ModuleGuid, Event.bIsInContact);
		}
	}
}
