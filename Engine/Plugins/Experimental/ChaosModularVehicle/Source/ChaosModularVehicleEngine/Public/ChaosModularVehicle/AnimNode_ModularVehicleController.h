// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "BoneContainer.h"
#include "BonePose.h"
#include "ModularVehicleAnimationInstance.h"
#include "BoneControllers/AnimNode_SkeletalControlBase.h"
#include "AnimNode_ModularVehicleController.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

/**
 *	Simple controller that replaces or adds to the translation/rotation of a single bone.
 */
USTRUCT()
struct FAnimNode_ModularVehicleController : public FAnimNode_SkeletalControlBase
{
	GENERATED_USTRUCT_BODY()

	UE_API FAnimNode_ModularVehicleController();

	// FAnimNode_Base interface
	UE_API virtual void GatherDebugData(FNodeDebugData& DebugData) override;
	// End of FAnimNode_Base interface

	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void EvaluateSkeletalControl_AnyThread(FComponentSpacePoseContext& Output, TArray<FBoneTransform>& OutBoneTransforms) override;
	UE_API virtual bool IsValidToEvaluate(const USkeleton* Skeleton, const FBoneContainer& RequiredBones) override;
	UE_API virtual void Initialize_AnyThread(const FAnimationInitializeContext& Context) override;
	// End of FAnimNode_SkeletalControlBase interface

private:
	// FAnimNode_SkeletalControlBase interface
	UE_API virtual void InitializeBoneReferences(const FBoneContainer& RequiredBones) override;
	// End of FAnimNode_SkeletalControlBase interface

	struct FModuleLookupData
	{
		int32 ModuleIndex;
		FBoneReference BoneReference;
	};

	TArray<FModuleLookupData> Modules;
	const FModularVehicleAnimationInstanceProxy* AnimInstanceProxy;
};

#undef UE_API
