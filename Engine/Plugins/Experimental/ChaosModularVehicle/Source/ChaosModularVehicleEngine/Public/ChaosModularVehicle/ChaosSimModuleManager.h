// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsInterfaceDeclaresCore.h"
#include "Engine/World.h"

#include "Chaos/SimCallbackInput.h"
#include "Chaos/SimCallbackObject.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

class FChaosSimModuleManagerAsyncCallback;
class UModularVehicleComponent;
class UModularVehicleBaseComponent;
class AHUD;
class FPhysScene_Chaos;
struct FChaosSimModuleManagerAsyncOutput;

// Helper class to store physics async output results
// Todo: generalize this as a Chaos helper template class
class FSimModuleOutputRecord
{
public:
	// Consumes an async output. Assumes the output is dated later than previously consumed outputs.
	// Uses move semantics, and will replace the oldest stored output.
	void ConsumeOutput(Chaos::TSimCallbackOutputHandle<FChaosSimModuleManagerAsyncOutput>&& Output);

	// Returns the oldest stored output
	const FChaosSimModuleManagerAsyncOutput* GetPreviousOutput() const;
	FChaosSimModuleManagerAsyncOutput* GetPreviousOutput();

	// Returns the most recent stored output
	const FChaosSimModuleManagerAsyncOutput* GetNextOutput() const;
	FChaosSimModuleManagerAsyncOutput* GetNextOutput();

	// Returns the start time (InternalTime) of the most recent output
	double GetLatestOutputStartTime() const;

	// Returns the interpolation factor
	double GetInterpolationFactor(double AtInternalTime) const;

	// Clears (and deletes) stored outputs
	void Clear();

private:
	Chaos::TSimCallbackOutputHandle<FChaosSimModuleManagerAsyncOutput> CachedOutput_0;
	Chaos::TSimCallbackOutputHandle<FChaosSimModuleManagerAsyncOutput> CachedOutput_1;
	bool bPreviousOutputIs0 = true;
};

class FChaosSimModuleManager
{
public:
	// Updated when vehicles need to recreate their physics state.
	// Used when values tweaked while the game is running.
	//static uint32 VehicleSetupTag;

	UE_API FChaosSimModuleManager(FPhysScene* PhysScene);
	UE_API ~FChaosSimModuleManager();

	static UE_API void OnPostWorldInitialization(UWorld* InWorld, const UWorld::InitializationValues);
	static UE_API void OnWorldCleanup(UWorld* InWorld, bool bSessionEnded, bool bCleanupResources);
	static UE_API void OnShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	/** Get Physics Scene */
	FPhysScene_Chaos& GetScene() const { return Scene; }

	/**
	 * Register a Physics vehicle for processing
	 */
	UE_API void AddVehicle(TWeakObjectPtr<UModularVehicleBaseComponent> Vehicle);

	/**
	 * Unregister a Physics vehicle from processing
	 */
	UE_API void RemoveVehicle(TWeakObjectPtr<UModularVehicleBaseComponent> Vehicle);

	/**
	 * Update vehicle tuning and other state such as input
	 */
	UE_API void ScenePreTick(FPhysScene* PhysScene, float DeltaTime);

	/** Detach this vehicle manager from a FPhysScene (remove delegates, remove from map etc) */
	UE_API void DetachFromPhysScene(FPhysScene* PhysScene);

	/** Update simulation of registered vehicles */
	UE_API void Update(FPhysScene* PhysScene, float DeltaTime);

	/** Post update step */
	UE_API void PostUpdate(FChaosScene* PhysScene);
	
	UE_API void OnNetDriverCreated(UWorld* InWorld, UNetDriver* InNetDriver);
	UE_API void RegisterNetTokenDataStores(UNetDriver* InNetDriver);

	/** Called on GT but before Physics thread runs - PT tick rate */
	UE_API void InjectInputs_External(int32 PhysicsStep, int32 NumSteps);

	UE_API void ParallelUpdateVehicles();

	/** Find a vehicle manager from an FPhysScene */
	static UE_API FChaosSimModuleManager* GetManagerFromScene(FPhysScene* PhysScene);

protected:
	UE_API void RegisterCallbacks(UWorld* InWorld);
	UE_API void UnregisterCallbacks();

private:
	/** Map of physics scenes to corresponding vehicle manager */
	static UE_API TMap<FPhysScene*, FChaosSimModuleManager*> SceneToModuleManagerMap;
	
	// The physics scene we belong to
	FPhysScene_Chaos& Scene;

	static UE_API bool GInitialized;

	// Test out new vehicle bass class
	TArray<TWeakObjectPtr<UModularVehicleBaseComponent>> CUVehicles;

	// callback delegates
	FDelegateHandle OnNetDriverCreatedHandle;
	FDelegateHandle OnPhysScenePreTickHandle;
	FDelegateHandle OnPhysScenePostTickHandle;

	static UE_API FDelegateHandle OnPostWorldInitializationHandle;
	static UE_API FDelegateHandle OnWorldCleanupHandle;

	// Async callback from the physics engine - we can run our simulation here
	FChaosSimModuleManagerAsyncCallback* AsyncCallback;	
	int32 Timestamp;
	int32 SubStepCount;

	// Record used to store the two most recent outputs which time surrounds the time we interpolate at
	FSimModuleOutputRecord OutputRecord;
};

#undef UE_API
