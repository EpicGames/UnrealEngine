// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "PhysicsPublic.h"
#include "Chaos/SimCallbackInput.h"
#include "Chaos/SimCallbackObject.h"
#include "Chaos/GeometryParticlesfwd.h"
#include "Iris/ReplicationSystem/NetTokenStructDefines.h"
#include "Iris/ReplicationSystem/StructNetTokenDataStore.h"
#include "Net/Iris/ReplicationSystem/StructNetTokenDataStoreHelper.h"
#include "Physics/NetworkPhysicsComponent.h"
#include "SimModule/SimulationModuleBase.h"
#include "SimModule/ModuleInput.h"

#include "ChaosSimModuleManagerAsyncCallback.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

class UModularVehicleComponent;
class UModularVehicleBaseComponent;
class FGeometryCollectionPhysicsProxy;
class UPackageMap;

DECLARE_STATS_GROUP(TEXT("ChaosSimModuleManager"), STATGROUP_ChaosSimModuleManager, STATGROUP_Advanced);


struct FSimModuleDebugParams
{
	bool EnableMultithreading = false;
	bool EnableNetworkStateData = true;
};

enum EChaosAsyncVehicleDataType : int8
{
	AsyncInvalid,
	AsyncDefault,
};


struct FModuleTransform
{
	int TransforIndex;
	FTransform Transform;
};

/** Vehicle inputs from the player controller */
USTRUCT()
struct FModularVehicleInputs
{
	GENERATED_USTRUCT_BODY()

		FModularVehicleInputs()
		: Reverse(false)
		, KeepAwake(false) 
		{
		}

	// Reversing state
	UPROPERTY()
	bool Reverse;

	// Keep vehicle awake
	UPROPERTY()
	bool KeepAwake;

	UPROPERTY()
	FModuleInputContainer Container;

};

/** Vehicle input data that will be used in the input history to be applied while simulating */
USTRUCT()
struct FNetworkModularVehicleInputs : public FNetworkPhysicsData
{
	GENERATED_USTRUCT_BODY()

	/** List of incoming control inputs coming from the local client */
	UPROPERTY()
	FModularVehicleInputs VehicleInputs;

	/**  Apply the data onto the network physics component */
	UE_API virtual void ApplyData(UActorComponent* NetworkComponent) const override;

	/**  Build the data from the network physics component */
	UE_API virtual void BuildData(const UActorComponent* NetworkComponent) override;

	/**  Serialize data function that will be used to transfer the struct across the network */
	UE_API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);

	/** Interpolate the data in between two inputs */
	UE_API virtual void InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData) override;

	/** Merge data when multiple inputs happen an same simulation tick*/
	UE_API virtual void MergeData(const FNetworkPhysicsData& FromData) override;

	/**  Decay data during resimulation by DecayAmount which increases over resimulation frames from 0.0 -> 1.0 when the input is being reused */
	UE_API virtual void DecayData(float DecayAmount) override;
};

template<>
struct TStructOpsTypeTraits<FNetworkModularVehicleInputs> : public TStructOpsTypeTraitsBase2<FNetworkModularVehicleInputs>
{
	enum
	{
		WithNetSerializer = true,
	};
};

USTRUCT()
struct FNetworkModularVehicleStateNetTokenData
{
	GENERATED_BODY()
	UPROPERTY()
	TArray<uint32> Hashes;
	UPROPERTY()
	TArray<int32> Indexes;
	UPROPERTY()
	TArray<bool> ModuleShouldSerialize;
	
	UE_NET_NETTOKEN_GENERATED_BODY(NetworkModularVehicleStateNetTokenData, CHAOSMODULARVEHICLEENGINE_API);

	UE_API uint64 GetUniqueKey() const;
	UE_API void Init(const Chaos::FModuleNetDataArray& ModuleData);
};

UE_NET_DECLARE_NAMED_NETTOKEN_STRUCT_SERIALIZERS(NetworkModularVehicleStateNetTokenData, CHAOSMODULARVEHICLEENGINE_API);

/** Vehicle state data that will be used in the state history to rewind the simulation */
USTRUCT()
struct FNetworkModularVehicleStates : public FNetworkPhysicsData
{
	GENERATED_BODY()

	inline static FName StashServerFrameKey = FName("ServerFrame");
	Chaos::FModuleNetDataArray ModuleData;
	/**  Apply the data onto the network physics component */
	UE_API virtual void ApplyData(UActorComponent* NetworkComponent) const override;

	/**  Build the data from the network physics component */
	UE_API virtual void BuildData(const UActorComponent* NetworkComponent) override;

	/**  Serialize data function that will be used to transfer the struct across the network */
	UE_API bool NetSerialize(FArchive& Ar, UPackageMap* Map, bool& bOutSuccess);
	UE_API bool DeltaNetSerialize(FArchive& Ar,  UPackageMap* Map, bool& bOutSuccess);

	/** Interpolate the data in between two states */
	UE_API virtual void InterpolateData(const FNetworkPhysicsData& MinData, const FNetworkPhysicsData& MaxData) override;
};

template<>
struct TStructOpsTypeTraits<FNetworkModularVehicleStates> : public TStructOpsTypeTraitsBase2<FNetworkModularVehicleStates>
{
	enum
	{
		WithNetSerializer = true,
	};
};

/**
 * Per Vehicle Output State from Physics Thread to Game Thread
 */
struct FPhysicsVehicleOutput
{
	FPhysicsVehicleOutput()
	{
	}

	~FPhysicsVehicleOutput()
	{
		Clean();
	}

	void Clean()
	{
		for (Chaos::FSimOutputData* Data : SimTreeOutputData)
		{
			if(Data)
			{
				delete Data;
			}
		}
		SimTreeOutputData.Empty();
	}

	const Chaos::FSimOutputData* GetOutputData(int InModuleGuid)
	{
		if (!SimTreeOutputData.IsEmpty())
		{
			int NumItems = SimTreeOutputData.Num();
			for (int I = 0; I < NumItems; I++)
			{
				if (SimTreeOutputData[I])
				{
					if (InModuleGuid == SimTreeOutputData[I]->ModuleGuid)
					{
						return SimTreeOutputData[I];
					}
				}
			}
		}

		return nullptr;
	}

	TArray<Chaos::FSimOutputData*> SimTreeOutputData;
	TArray<Chaos::FCreatedModule> NewlyCreatedModuleGuids;
};

struct FPhysicsModularVehicleTraits
{
	using InputsType = FNetworkModularVehicleInputs;
	using StatesType = FNetworkModularVehicleStates;
};

// #TBD
struct FGameStateInputs
{
	FModuleInputContainer StateInputContainer;
};

UENUM()
enum class ETraceType : uint8
{
	/** Use ray to determine suspension length to ground */
	Raycast		UMETA(DisplayName = "Raycast"),

	/** Use sphere to determine suspension length to ground */
	Spherecast	UMETA(DisplayName = "Spherecast"),
};

/**
 * Per Vehicle input State from Game Thread to Physics Thread
 */
struct FPhysicsModularVehicleInputs
{
	FPhysicsModularVehicleInputs()
		: CollisionChannel(ECollisionChannel::ECC_WorldDynamic)
		, TraceParams()
		, TraceCollisionResponse()
		, TraceType(ETraceType::Raycast)
		, CurrentTimeDilation(1.0f)
	{
	}
	mutable FNetworkModularVehicleInputs NetworkInputs;
	mutable ECollisionChannel CollisionChannel;
	mutable FCollisionQueryParams TraceParams;
	mutable FCollisionResponseContainer TraceCollisionResponse;
	mutable ETraceType TraceType;
	mutable FGameStateInputs StateInputs;
	mutable float CurrentTimeDilation;
};

/**
 * Per Vehicle Input State from Game Thread to Physics Thread
 */
struct FModularVehicleAsyncInput
{
	FModularVehicleAsyncInput(EChaosAsyncVehicleDataType InType = EChaosAsyncVehicleDataType::AsyncInvalid)
		: Type(InType)
		, Vehicle(nullptr)
	{
		Proxy = nullptr;	//indicates async/sync task not needed
	}

	virtual ~FModularVehicleAsyncInput() = default;

	/**
	* Vehicle simulation running on the Physics Thread
	*/
	UE_API virtual TUniquePtr<struct FModularVehicleAsyncOutput> Simulate(UWorld* World, const float DeltaSeconds, const float TotalSeconds, bool& bWakeOut) const;

	UE_API virtual void OnContactModification(Chaos::FCollisionContactModifier& Modifier) const;
	UE_API virtual void ApplyDeferredForces() const;
	UE_API virtual void ProcessInputs();

	void SetVehicle(UModularVehicleBaseComponent* VehicleIn) { Vehicle = VehicleIn; }
	UModularVehicleBaseComponent* GetVehicle() const { return Vehicle; }

	const EChaosAsyncVehicleDataType Type;
	IPhysicsProxyBase* Proxy;
	bool bIsLocallyControlled = false;

	FPhysicsModularVehicleInputs PhysicsInputs;

private:

	UModularVehicleBaseComponent* Vehicle;

};

struct FChaosSimModuleManagerAsyncInput : public Chaos::FSimCallbackInput
{
	TArray<TUniquePtr<FModularVehicleAsyncInput>> VehicleInputs;

	TWeakObjectPtr<UWorld> World;
	int32 Timestamp = INDEX_NONE;

	void Reset()
	{
		VehicleInputs.Reset();
		World.Reset();
	}
};

/**
 * Async Output Data
 */
struct FModularVehicleAsyncOutput
{
	const EChaosAsyncVehicleDataType Type;
	bool bValid;	// indicates no work was done
	FPhysicsVehicleOutput VehicleSimOutput;
	UModularVehicleBaseComponent* Vehicle;

	FModularVehicleAsyncOutput(EChaosAsyncVehicleDataType InType = EChaosAsyncVehicleDataType::AsyncInvalid)
		: Type(InType)
		, bValid(false)
		, Vehicle(nullptr)
	{ }

	virtual ~FModularVehicleAsyncOutput()
	{
		VehicleSimOutput.Clean();
	}
};


/**
 * Async Output for all of the vehicles handled by this Vehicle Manager
 */
struct FChaosSimModuleManagerAsyncOutput : public Chaos::FSimCallbackOutput
{
	TArray<TUniquePtr<FModularVehicleAsyncOutput>> VehicleOutputs;
	int32 Timestamp = INDEX_NONE;

	void Reset()
	{
		VehicleOutputs.Reset();
	}
};

/**
 * Async callback from the Physics Engine where we can perform our vehicle simulation
 */
class FChaosSimModuleManagerAsyncCallback : public Chaos::TSimCallbackObject<FChaosSimModuleManagerAsyncInput, FChaosSimModuleManagerAsyncOutput
	, Chaos::ESimCallbackOptions::Presimulate | Chaos::ESimCallbackOptions::Rewind | Chaos::ESimCallbackOptions::ContactModification>
{
public:
	UE_API virtual FName GetFNameForStatId() const override;
private:
	UE_API virtual void ProcessInputs_Internal(int32 PhysicsStep) override;
	UE_API virtual void OnPreSimulate_Internal() override;
	UE_API virtual void OnContactModification_Internal(Chaos::FCollisionContactModifier& Modifications) override;
};

#undef UE_API
