// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PhysicsEngine/ClusterUnionComponent.h"
#include "ChaosModularVehicle/ModularVehicleSocket.h"
#include "PhysicsProxy/ClusterUnionPhysicsProxy.h"

#include "ClusterUnionVehicleComponent.generated.h"

UCLASS(MinimalAPI)
class UClusterUnionVehicleComponent : public UClusterUnionComponent
{
	GENERATED_BODY()
public:

	CHAOSMODULARVEHICLEENGINE_API UClusterUnionVehicleComponent(const FObjectInitializer& ObjectInitializer);

	//~ Begin USceneComponent Interface.
	CHAOSMODULARVEHICLEENGINE_API virtual bool HasAnySockets() const override;
	CHAOSMODULARVEHICLEENGINE_API virtual bool DoesSocketExist(FName InSocketName) const override;
	CHAOSMODULARVEHICLEENGINE_API virtual FTransform GetSocketTransform(FName InSocketName, ERelativeTransformSpace TransformSpace = RTS_World) const override;
	CHAOSMODULARVEHICLEENGINE_API virtual void QuerySupportedSockets(TArray<FComponentSocketDescription>& OutSockets) const override;
	//~ Begin USceneComponent Interface.

	UPROPERTY(EditAnywhere, meta = (BlueprintSpawnableComponent), Category = "ModularVehicle")
	TArray<FModularVehicleSocket> Sockets;

};