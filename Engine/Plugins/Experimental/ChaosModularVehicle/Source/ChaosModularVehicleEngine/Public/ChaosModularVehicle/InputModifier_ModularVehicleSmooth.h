// Copyright Epic Games, Inc. All Rights Reserved.

 #pragma once

#include "CoreMinimal.h"
#include "InputModifiers.h"
#include "InputActionValue.h"
#include "InputModifier_ModularVehicleSmooth.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

/**
 * UInputModifier_ModularVehicleSmooth
 *
 * Add this as a Modifier on any float‐based Action (e.g. “Steer”) to get a
 * smoother, “lagged” feel. To match the behaviour of the OG chaos vehicle plugin
 */
UCLASS(MinimalAPI, BlueprintType, meta = (DisplayName = "Modular Vehicle Smooth Modifier"))
class UInputModifier_ModularVehicleSmooth : public UInputModifier
{
	GENERATED_BODY()

public:
	/**
	 * How quickly to “catch up” to the raw input.
	 * Higher numbers = snappier response; lower = more lag/smoothing.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Smoothing")
	float RiseRate = 10.0f;

protected:
	UE_API virtual FInputActionValue ModifyRaw_Implementation(const UEnhancedPlayerInput* PlayerInput, FInputActionValue CurrentValue, float DeltaTime) override;

	float PreviousValue = 0.0f;
};

#undef UE_API
