// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/ModuleInput.h"
#include "InputProducer.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

/** The default input producer that takes real input from the player and provides it to the simulation */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UVehicleDefaultInputProducer : public UVehicleInputProducerBase
{
	GENERATED_BODY()

public:

	/* initialize the input buffer container */
	UE_API virtual void InitializeContainer(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut, EModuleInputQuantizationType InInputQuantizationType) override;

	/** capture input at game thread frequency */
	UE_API virtual void BufferInput(const FInputNameMap& InNameMap, const FName InName, const FModuleInputValue& InValue, EModuleInputBufferActionType BufferAction) override;

	/** produce input for PT simulation at PT frequency */
	UE_API virtual void ProduceInput(int32 PhysicsStep, int32 NumSteps, const FInputNameMap& InNameMap, FModuleInputContainer& InOutContainer) override;

	FModuleInputContainer MergedInput;
};


/** Example input generator, generates random input inot a per frame buffer then replays from the buffer, looping back to the start when the bugger is exhausted */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UVehiclePlaybackInputProducer : public UVehicleInputProducerBase
{
	GENERATED_BODY()

public:

	/* initialize the input buffer containers */
	UE_API virtual void InitializeContainer(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut, EModuleInputQuantizationType InInputQuantizationType) override;

	/** capture input at game thread frequency */
	UE_API virtual void BufferInput(const FInputNameMap& InNameMap, const FName InName, const FModuleInputValue& InValue, EModuleInputBufferActionType BufferAction) override;

	/** produce input for PT simulation at PT frequency */
	UE_API virtual void ProduceInput(int32 PhysicsStep, int32 NumSteps, const FInputNameMap& InNameMap, FModuleInputContainer& InOutContainer) override;

	/** Special case override for providing test input straight onto the physics thread */
	virtual TArray<FModuleInputContainer>* GetTestInputBuffer() { return &PlaybackBuffer; }

	/** Special case override for providing test input straight onto the physics thread */
	virtual bool IsLoopingTestInputBuffer() { return true; }

	TArray<FModuleInputContainer> PlaybackBuffer;
	int32 BufferLength = 150;
	int32 StartStep = 0;
};


/** Example input generator, generates random input on the fly for the PT */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UVehicleRandomInputProducer : public UVehicleInputProducerBase
{
	GENERATED_BODY()

public:

	/* initialize the input buffer containers */
	UE_API virtual void InitializeContainer(TArray<FModuleInputSetup>& SetupData, FInputNameMap& NameMapOut, EModuleInputQuantizationType InInputQuantizationType) override;

	/** capture input at game thread frequency */
	UE_API virtual void BufferInput(const FInputNameMap& InNameMap, const FName InName, const FModuleInputValue& InValue, EModuleInputBufferActionType BufferAction) override;

	/** produce input for PT simulation at PT frequency */
	UE_API virtual void ProduceInput(int32 PhysicsStep, int32 NumSteps, const FInputNameMap& InNameMap, FModuleInputContainer& InOutContainer) override;

	FModuleInputContainer PlaybackContainer;
	int32 ChangeInputFrequency = 10;
};

#undef UE_API
