// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimInstanceProxy.h"
#include "ModularVehicleAnimationInstance.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

class UModularVehicleBaseComponent;

struct FModuleAnimationData
{
	FName BoneName;
	FRotator RotOffset;
	FVector LocOffset;
	uint16 Flags;
};

/** Proxy override for this UAnimInstance-derived class */
USTRUCT()
	struct FModularVehicleAnimationInstanceProxy : public FAnimInstanceProxy
{
	GENERATED_BODY()

		FModularVehicleAnimationInstanceProxy()
		: FAnimInstanceProxy()
	{
	}

	FModularVehicleAnimationInstanceProxy(UAnimInstance* Instance)
		: FAnimInstanceProxy(Instance)
	{
	}

public:

	UE_API void SetModularVehicleComponent(const UModularVehicleBaseComponent* InWheeledVehicleComponent);

	/** FAnimInstanceProxy interface begin*/
	UE_API virtual void PreUpdate(UAnimInstance* InAnimInstance, float DeltaSeconds) override;
	/** FAnimInstanceProxy interface end*/

	const TArray<FModuleAnimationData>& GetModuleAnimData() const
	{
		return ModuleInstances;
	}

private:
	TArray<FModuleAnimationData> ModuleInstances;
};

UCLASS(MinimalAPI, transient)
	class UModularVehicleAnimationInstance : public UAnimInstance
{
	GENERATED_UCLASS_BODY()

		/** Makes a montage jump to the end of a named section. */
		UFUNCTION(BlueprintCallable, Category = "Animation")
		UE_API class AModularVehicleClusterPawn* GetVehicle();

public:
	TArray<TArray<FModuleAnimationData>> ModuleData;

public:
	void SetModularVehicleComponent(const UModularVehicleBaseComponent* InWheeledVehicleComponent)
	{
		ModularVehicleComponent = InWheeledVehicleComponent;
		AnimInstanceProxy.SetModularVehicleComponent(InWheeledVehicleComponent);
	}

	const UModularVehicleBaseComponent* GetModularVehicleComponent() const
	{
		return ModularVehicleComponent;
	}

private:
	/** UAnimInstance interface begin*/
	UE_API virtual void NativeInitializeAnimation() override;
	UE_API virtual FAnimInstanceProxy* CreateAnimInstanceProxy() override;
	UE_API virtual void DestroyAnimInstanceProxy(FAnimInstanceProxy* InProxy) override;
	/** UAnimInstance interface end*/

	FModularVehicleAnimationInstanceProxy AnimInstanceProxy;

	UPROPERTY(transient)
	TObjectPtr<const UModularVehicleBaseComponent> ModularVehicleComponent;
};


#undef UE_API
