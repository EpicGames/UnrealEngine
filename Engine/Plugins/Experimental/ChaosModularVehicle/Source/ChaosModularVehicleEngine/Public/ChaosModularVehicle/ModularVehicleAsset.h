// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"

#include "ChaosModularVehicle/ModularSimCollection.h"

#include "ModularVehicleAsset.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

class UModularVehicleAsset;

/**
*	FModularVehicleAssetEdit
*     Structured RestCollection access where the scope
*     of the object controls serialization back into the
*     dynamic collection
*
*/
class FModularVehicleAssetEdit
{
public:
	/**
	 * @param UModularVehicleAsset	The FAsset to edit
	 */
	UE_API FModularVehicleAssetEdit(UModularVehicleAsset* InAsset);
	UE_API ~FModularVehicleAssetEdit();

	UE_API UModularVehicleAsset* GetAsset();

private:
	UModularVehicleAsset* Asset;
};

/**
* UModularVehicleAsset (UObject)
*
* UObject wrapper for the FModularVehicleAsset
*
*/
UCLASS(MinimalAPI, customconstructor)
class UModularVehicleAsset : public UObject
{
	GENERATED_UCLASS_BODY()
	friend class FModularVehicleAssetEdit;

public:

	UE_API UModularVehicleAsset(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	FModularVehicleAssetEdit EditRestCollection() { return FModularVehicleAssetEdit(this); }

	UE_API void Serialize(FArchive& Ar);

#if WITH_EDITORONLY_DATA
	/** Information for thumbnail rendering */
	UPROPERTY(VisibleAnywhere, Instanced, AdvancedDisplay, Category = GeometryCollection)
	TObjectPtr<class UThumbnailInfo> ThumbnailInfo;
#endif // WITH_EDITORONLY_DATA

private:
	TSharedPtr<FModularSimCollection, ESPMode::ThreadSafe> ModularSimCollection;
};

#undef UE_API
