// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosModularVehicle/ChaosSimModuleManagerAsyncCallback.h"
#include "SimModule/SimModulesInclude.h"
#include "SimModule/ModuleInput.h"
#include "ChaosModularVehicle/InputProducer.h"
#include "ChaosModularVehicle/ModularVehicleSimulationCU.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Components/PrimitiveComponent.h"
#include "Components/ActorComponent.h"
#include "Containers/Map.h"
#include "UObject/ObjectKey.h"

#include "ModularVehicleBaseComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogModularBase, Log, All);

struct FModularVehicleAsyncInput;
struct FChaosSimModuleManagerAsyncOutput;
class UClusterUnionComponent;
namespace Chaos
{
	class FSimTreeUpdates;
	struct FSimOutputData;
}

USTRUCT()
struct FVehicleComponentData
{
	GENERATED_BODY()

	int Guid = -1;

	// Cached reference to the visual component to animate (if different from sim component)
	UPROPERTY()
	TObjectPtr<USceneComponent> VisualComponentToAnimate = nullptr;

};

/** Additional replicated state */
USTRUCT()
struct FModularReplicatedState : public FModularVehicleInputs
{
	GENERATED_USTRUCT_BODY()

	FModularReplicatedState() : FModularVehicleInputs()
	{
	}

};


USTRUCT()
struct FConstructionData
{
	GENERATED_USTRUCT_BODY()

	UPROPERTY()
	TObjectPtr<USceneComponent> Component = nullptr;

	UPROPERTY()
	int32 ConstructionIndex = INDEX_NONE;
};


USTRUCT()
struct FModuleAnimationSetup
{
	GENERATED_USTRUCT_BODY()

	FModuleAnimationSetup(FName BoneNameIn, int TransformIndexIn, int GuidIn = INDEX_NONE)
		: BoneName(BoneNameIn)
		, RotOffset(FRotator::ZeroRotator)
		, LocOffset(FVector::ZeroVector)
		, CombinedRotation(FQuat::Identity)
		, AnimFlags(0)
		, TransformIndex(TransformIndexIn)
		, ModuleGUID(GuidIn)
		, InitialRotOffset(FQuat::Identity)
		, InitialLocOffset(FVector::ZeroVector)
	{
	}

	FModuleAnimationSetup() 
		: BoneName(NAME_None)
		, RotOffset(FRotator::ZeroRotator)
		, LocOffset(FVector::ZeroVector)
		, CombinedRotation(FQuat::Identity)
		, AnimFlags(0)
		, TransformIndex(INDEX_NONE)
		, ModuleGUID(INDEX_NONE)
		, InitialRotOffset(FQuat::Identity)
		, InitialLocOffset(FVector::ZeroVector)
	{
	}

	FName BoneName;		// required for skeletal mesh
	FRotator RotOffset;
	FVector LocOffset;
	FQuat CombinedRotation;
	uint16 AnimFlags;
	int TransformIndex; // required for non skeletal mesh case
	int ModuleGUID;
	FQuat InitialRotOffset;
	FVector InitialLocOffset;

};


UCLASS(MinimalAPI, ClassGroup = (Physics), meta = (BlueprintSpawnableComponent), hidecategories = (PlanarMovement, "Components|Movement|Planar", Activation, "Components|Activation"))
class UModularVehicleBaseComponent : public UPawnMovementComponent
{
	GENERATED_UCLASS_BODY()

	UE_API ~UModularVehicleBaseComponent();

	friend struct FModularVehicleAsyncInput;
	friend struct FModularVehicleAsyncOutput;

	friend struct FNetworkModularVehicleInputs;
	friend struct FNetworkModularVehicleStates;

	friend class FModularVehicleManager;
	friend class FChaosSimModuleManagerAsyncCallback;

	friend class FModularVehicleBuilder;
public:

	using FInputNameMap = TMap<FName, int>;

	UE_API APlayerController* GetPlayerController() const;
	UE_API bool IsLocallyControlled() const;
	void SetTreeProcessingOrder(ESimTreeProcessingOrder TreeProcessingOrderIn) { TreeProcessingOrder = TreeProcessingOrderIn; }
	ESimTreeProcessingOrder GetTreeProcessingOrder() { return TreeProcessingOrder; }

	UE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;

	virtual bool ShouldCreatePhysicsState() const override { return true; }
	UE_API virtual void OnCreatePhysicsState() override;
	UE_API virtual void OnDestroyPhysicsState() override;
	UE_API virtual void SetClusterComponent(UClusterUnionComponent* InPhysicalComponent);

	UE_API virtual void BeginPlay() override;
	UE_API virtual void TickComponent(float DeltaTime, ELevelTick TickType, FActorComponentTickFunction* ThisTickFunction) override;

	UE_API void ProduceInput(int32 PhysicsStep, int32 NumSteps, FModularVehicleAsyncInput* AsyncInput);

	// Create sim components hierarchy from parent component - used by original cluster code union path
	UE_API virtual void CreateAssociatedSimComponents(USceneComponent* ParentComponent, USceneComponent* AttachedComponent, int ParentIndex, int TransformIndex, Chaos::FSimTreeUpdates& TreeUpdatesOut);

	// Create sim components in hierarchy from parent component - used by code path that does not rely on cluster union
	UE_API virtual void CreateIndependentSimComponents(USceneComponent* RootComponent, USceneComponent* AttachedComponent, int ParentIndex, int TransformIndex, Chaos::FSimTreeUpdates& TreeUpdatesOut);

	UE_API void PreTickGT(float DeltaTime);
	UE_API void UpdateState(float DeltaTime);
	UE_API void SetCurrentAsyncData(FChaosSimModuleManagerAsyncOutput* CurOutput, FChaosSimModuleManagerAsyncOutput* NextOutput, float Alpha, int32 VehicleManagerTimestamp);

	UE_API void ParallelUpdate(const Chaos::FCreatedModules& ModuleEvents);
	UE_API void Update(float DeltaTime);
	UE_API void PostUpdate();
	UE_API void FinalizeSimCallbackData(FChaosSimModuleManagerAsyncInput& Input);

	/** handle stand-alone and networked mode control inputs */
	UE_API void ProcessControls(float DeltaTime);

	UE_API void ShowDebugInfo(AHUD* HUD, UCanvas* Canvas, const FDebugDisplayInfo& DisplayInfo, float& YL, float& YPos);

	TUniquePtr<FPhysicsVehicleOutput>& PhysicsVehicleOutput()
	{
		return PVehicleOutput;
	}

	UE_API const FTransform& GetComponentTransform() const;
	
	// native (fast, low overhead) versions 
	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnModuleAddedNative, const FName&, int, int);
	FOnModuleAddedNative OnModuleAddedNativeEvent;

	DECLARE_MULTICAST_DELEGATE_ThreeParams(FOnModuleRemovedNative, const FName&, int, int);
	FOnModuleRemovedNative OnModuleRemovedNativeEvent;

	// standard events
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnModuleAdded, const FName&, SimType, int, Guid, int, TreeIndex);
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnModuleAdded OnModuleAddedEvent;

	DECLARE_DYNAMIC_MULTICAST_DELEGATE_ThreeParams(FOnModuleRemoved, const FName&, SimType, int, Guid, int, TreeIndex);
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnModuleRemoved OnModuleRemovedEvent;

	/** Use to naturally decelerate linear velocity of objects */
	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	float LinearDamping;

	/** Use to naturally decelerate angular velocity of objects */
	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	float AngularDamping;

	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	struct FCollisionResponseContainer SuspensionTraceCollisionResponses;

	/** Collision channel to use for the suspension trace. */
	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	TEnumAsByte<ECollisionChannel> SuspensionCollisionChannel;

	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	bool bSuspensionTraceComplex;

	/** Wheel suspension trace type, defaults to ray trace */
	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	ETraceType TraceType;

	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	bool bKeepVehicleAwake;

	/** Adds any associated simulation components to the ModularVehicleSimulation */
	UFUNCTION()
	UE_API void AddComponentToSimulation(UPrimitiveComponent* Component, const TArray<FClusterUnionBoneData>& BonesData, const TArray<FClusterUnionBoneData>& RemovedBoneIDs, bool bIsNew);

	/** Removes any associated simulation components from the ModularVehicleSimulation */
	UFUNCTION()
	UE_API void RemoveComponentFromSimulation(UPrimitiveComponent* Component, const TArray<FClusterUnionBoneData>& RemovedBonesData);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void SetLocallyControlled(bool bLocallyControlledIn);

	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	TSubclassOf<UVehicleInputProducerBase> InputProducerClass;

	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	EModuleInputQuantizationType InputQuantizationType = EModuleInputQuantizationType::Default_16Bits;
	
	UFUNCTION()
	UE_API virtual void OnModuleInitialized(const FName& SimType, int Guid, int TreeIndex);

	UFUNCTION()
	UE_API virtual void OnModuleRemoved(const FName& SimType, int Guid, int TreeIndex);

	// CONTROLS
	// 
	//void SetInput(const FName& Name, const FModuleInputValue& Value);
	// Control inputs are automatically quantized for networking
	UE_API void SetInput(const FName& Name, const bool Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);
	UE_API void SetInput(const FName& Name, const int32 Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);
	UE_API void SetInput(const FName& Name, const double Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);
	UE_API void SetInput(const FName& Name, const FVector2D& Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);
	UE_API void SetInput(const FName& Name, const FVector& Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);

	// The state inputs are not automatically quantized
	UE_API void SetState(const FName& Name, const bool Value);
	UE_API void SetState(const FName& Name, const int32 Value);
	UE_API void SetState(const FName& Name, const double Value);
	UE_API void SetState(const FName& Name, const FVector2D& Value);
	UE_API void SetState(const FName& Name, const FVector& Value);

	UE_API void SetupInputConfiguration();

	// Sets the input producer class and creates an input producer if one doesn't exist, or if forced to create an instance with the new class.
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void SetInputProducerClass(TSubclassOf<UVehicleInputProducerBase> InInputProducerClass, bool bForceNewInstance = false);

	TObjectPtr<UVehicleInputProducerBase> GetInputProducer() { return InputProducer; }

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void SetInputBool(const FName Name, const bool Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void SetInputInteger(const FName Name, const int32 Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void SetInputAxis1D(const FName Name, const double Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void SetInputAxis2D(const FName Name, const FVector2D Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void SetInputAxis3D(const FName Name, const FVector Value, EModuleInputBufferActionType BufferAction = EModuleInputBufferActionType::Override);

	/** Set the gear directly */
	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void SetGearInput(int32 Gear);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API int32 GetCurrentGear();

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API bool IsReversing();

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void AddActorsToIgnore(TArray<AActor*>& ActorsIn);

	UFUNCTION(BlueprintCallable, Category = "Game|Components|ModularVehicle")
	UE_API void RemoveActorsToIgnore(TArray<AActor*>& ActorsIn);

	// Bypass the need for a controller in order for the controls to be processed.
	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	bool bRequiresControllerForInputs;

	/** Grab nearby components and add them to the cluster union representing the vehicle */
	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	bool bAutoAddComponentsFromWorld;

	/** The size of the overlap box testing for nearby components in the world  */
	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle", meta = (EditCondition = "bAutoAddComponentsToCluster"))
	FVector AutoAddOverlappingBoxSize;

	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle", meta = (EditCondition = "bAutoAddComponentsToCluster"))
	int32 DelayClusteringCount;

	/*** Map simulation component to our vehicle setup data */
	UPROPERTY()
	TMap<TObjectPtr<USceneComponent>, FVehicleComponentData> ComponentToPhysicsObjects;

	/*** Map physics sim GUID to actor component */
	TMap<int, TWeakObjectPtr<USceneComponent>> PhysicsGuidToComponent;

	UClusterUnionComponent* ClusterUnionComponent;

	/** Set all channels to the specified response - for wheel raycasts */
	void SetWheelTraceAllChannels(ECollisionResponse NewResponse)
	{
		SuspensionTraceCollisionResponses.SetAllChannels(NewResponse);
	}

	/** Set the response of this body to the supplied settings - for wheel raycasts */
	void SetWheelTraceResponseToChannel(ECollisionChannel Channel, ECollisionResponse NewResponse)
	{
		SuspensionTraceCollisionResponses.SetResponse(Channel, NewResponse);
	}

	TArray<FModuleAnimationSetup>& AccessModuleAnimationSetups() { return ModuleAnimationSetups; }
	const TArray<FModuleAnimationSetup>& GetModuleAnimationSetups() const { return ModuleAnimationSetups; }

	UE_API const Chaos::FSimOutputData* GetOutputData(int ModuleGuid);

	void SetPhysicsProxy(IPhysicsProxyBase* PhysicsProxy) { CachedPhysicsProxy = PhysicsProxy; }
	void SetPhysicsProxyIfNotAlreadySpecified(IPhysicsProxyBase* PhysicsProxy)
	{ 
		if (CachedPhysicsProxy == nullptr) 
		{
			CachedPhysicsProxy = PhysicsProxy; 
		}
	}

protected:

	UE_API void CreateVehicleSim();
	UE_API void DestroyVehicleSim();
	UE_API void UpdatePhysicalProperties();
	UE_API void AddOverlappingComponentsToCluster();
	UE_API void AddComponentToCluster(USceneComponent* Component);
	UE_API bool AddComponentsFromOwnedActor();
	UE_API void SetupSkeletalAnimationStructure();
	UE_API void UpdateNonSkeletalAnimations();
	UE_API void AssimilateComponentInputs(TArray<FModuleInputSetup>& OutCombinedInputs);
	UE_API void CacheRootPhysicsObject(IPhysicsProxyBase* Proxy);

	UE_API void AddSimulationComponentsFromOwnedActor(); // This version does not require cluster unions or geometry collections as parent components

	UE_API virtual void AddComponentToSimulationImpl(UPrimitiveComponent* Component, const TArray<FClusterUnionBoneData>& BonesData, const TArray<FClusterUnionBoneData>& RemovedBoneIDs, bool bIsNew);
	UE_API virtual void RemoveComponentFromSimulationImpl(UPrimitiveComponent* Component, const TArray<FClusterUnionBoneData>& RemovedBonesData);

	UE_API void ActionTreeUpdates(Chaos::FSimTreeUpdates* NextTreeUpdates);

	UE_API void SetCurrentAsyncDataInternal(FChaosSimModuleManagerAsyncOutput* CurOutput, FChaosSimModuleManagerAsyncOutput* NextOutput, float Alpha, int32 VehicleManagerTimestamp);
	UE_API int32 FindParentsLastSimComponent(const USceneComponent* AttachedComponent);

	UE_API IPhysicsProxyBase* GetPhysicsProxy() const;
	UE_API int GenerateNewGuid();

	UE_API int32 FindComponentAddOrder(USceneComponent* InComponent);
	UE_API void CreateOrUpdateComponentAddOrder(USceneComponent* InComponent, int32 InComponentAddOrder);
	UE_API bool FindAndRemoveNextPendingUpdate(int32 NextIndex, Chaos::FSimTreeUpdates* OutData);

	UE_API void BroadcastModuleAddedEvent(const FName& SimType, int Guid, int TreeIndex);
	UE_API void BroadcastModuleRemovedEvent(const FName& SimType, int Guid, int TreeIndex);

	UE_API Chaos::FSimOutputData* FindModuleOutputFromGuid(const FPhysicsVehicleOutput& OutputContainer, int Guid) const;

	// replicated state of vehicle 
	UPROPERTY(Transient, Replicated)
	FModularReplicatedState ReplicatedState;

	// latest gear selected
	UPROPERTY(Transient)
	int32 GearInput;

	// The currently selected gear
	UPROPERTY(Transient)
	int32 CurrentGear;

	// The engine RPM
	UPROPERTY(Transient)
	float EngineRPM;

	// The engine Torque
	UPROPERTY(Transient)
	float EngineTorque;

	UPROPERTY()
	TObjectPtr<UNetworkPhysicsComponent> NetworkPhysicsComponent = nullptr;

public:

	UPROPERTY(EditAnywhere, Category = VehicleInput, meta=(TitleProperty="{Name} | {Type}"))
	TArray<FModuleInputSetup> InputConfig;

	UPROPERTY(EditAnywhere, Category = VehicleState, meta=(TitleProperty="{Name} | {Type}"))
	TArray<FModuleInputSetup> StateInputConfiguration;
		
	UPROPERTY(EditAnywhere, Category = "Game|Components|ModularVehicle")
	TEnumAsByte<ESimTreeProcessingOrder> TreeProcessingOrder = ESimTreeProcessingOrder::LeafFirst;

	UPROPERTY(Transient, Replicated)
	TArray<FConstructionData> ConstructionDatas;

	/** Pass current state to server */
	UFUNCTION(reliable, server, WithValidation)
	UE_API void ServerUpdateState(const FModuleInputContainer& InputsIn, bool KeepAwake);

	UE_API void AddInput(FModuleInputSetup InputSetup);

	UE_API void LogInputSetup();

	UE_API int AddSimModule(Chaos::ISimulationModuleBase* NewModule, const FTransform& ComponentTransform, int ParentIndex, int TransformIndex);
	UE_API void RemoveSimModule(int ModuleGuid);
	UE_API void FinalizeModuleUpdates();

	Chaos::FSimTreeUpdates StoredTreeUpdates;

	TArray<AActor*> ActorsToIgnore;
	EChaosAsyncVehicleDataType CurAsyncType;
	FModularVehicleAsyncInput* CurAsyncInput;
	struct FModularVehicleAsyncOutput* CurAsyncOutput;
	struct FModularVehicleAsyncOutput* NextAsyncOutput;
	float OutputInterpAlpha;

	struct FAsyncOutputWrapper
	{
		int32 Idx;
		int32 Timestamp;

		FAsyncOutputWrapper()
			: Idx(INDEX_NONE)
			, Timestamp(INDEX_NONE)
		{
		}
	};

	TUniquePtr<FPhysicsVehicleOutput> PVehicleOutput;	/* physics simulation data output from the async physics thread */
	TUniquePtr<FModularVehicleSimulation> VehicleSimulationPT;	/* simulation code running on the physics thread async callback */

private:

	int NextTransformIndex = 0; // is there a better way, getting from size of map/array?
	UPrimitiveComponent* MyComponent = nullptr;

	bool bUsingNetworkPhysicsPrediction = false;
	float PrevSteeringInput = 0.0f;

	int32 LastComponentAddIndex = INDEX_NONE;
	TMap<TObjectKey<USceneComponent>, Chaos::FSimTreeUpdates> PendingTreeUpdates;

	int32 NextConstructionIndex = 0;

	int32 ClusteringCount = 0;

	bool bIsLocallyControlled;

	TArray<FModuleAnimationSetup> ModuleAnimationSetups;

	FInputNameMap InputNameMap;	// map input name to input container array index

	UPROPERTY(Transient)
	TObjectPtr<UVehicleInputProducerBase> InputProducer = nullptr;

	FModuleInputContainer InputsContainer;

	FInputNameMap StateNameMap;
	FModuleInputContainer StateInputContainer;

	Chaos::FPhysicsObjectHandle RootPhysicsObject = nullptr;
	mutable IPhysicsProxyBase* CachedPhysicsProxy = nullptr;
};

#undef UE_API
