// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/SimModuleTree.h"

class UModularVehicleBaseComponent;

class FModularVehicleBuilder
{
public:

	static CHAOSMODULARVEHICLEENGINE_API void GenerateSimTree(UModularVehicleBaseComponent* ModularVehicle);
	static CHAOSMODULARVEHICLEENGINE_API void FixupTreeLinks(TUniquePtr<Chaos::FSimModuleTree>& SimModuleTree);
};
