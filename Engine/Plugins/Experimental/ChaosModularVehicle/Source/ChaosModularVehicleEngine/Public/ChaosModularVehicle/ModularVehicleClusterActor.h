// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"

#include "ModularVehicleClusterActor.generated.h"

class UClusterUnionVehicleComponent;
class UModularVehicleBaseComponent;

UCLASS(MinimalAPI)
class AModularVehicleClusterActor: public AActor
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION()
	UClusterUnionVehicleComponent* GetClusterUnionComponent() const { return ClusterUnionVehicleComponent; }

	UFUNCTION()
	UModularVehicleBaseComponent* GetVehicleSimulationComponent() const { return VehicleSimComponent; }

	/* VehicleSpecificClusterComponent */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Vehicle, meta = (ExposeFunctionCategories = "Components|ModularVehicle", AllowPrivateAccess = "true"))
	TObjectPtr<UClusterUnionVehicleComponent> ClusterUnionVehicleComponent;

	/* ModularVehicleComponent */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Vehicle, meta = (ExposeFunctionCategories = "Components|ModularVehicle", AllowPrivateAccess = "true"))
	TObjectPtr<UModularVehicleBaseComponent> VehicleSimComponent;

};
