// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/StringConv.h"
#include "HAL/FileManager.h"


	static bool WriteCustomReport(FString FileName, TArray<FString>& FileLines)
	{
		// Has a report been generated
		bool ReportGenerated = false;

		// Ensure we have a log to write
		if (FileLines.Num())
		{
			// Create the file name		
			//FString FileLocation = FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() + TEXT("Reports/"));
			//FString FullPath = FString::Printf(TEXT("%s%s"), *FileLocation, *FileName);
			FString FullPath = FileName;

			// save file
			FArchive* LogFile = IFileManager::Get().CreateFileWriter(*FullPath, FILEWRITE_NoReplaceExisting | FILEWRITE_Append);

			if (LogFile != NULL)
			{
				for (int32 Index = 0; Index < FileLines.Num(); ++Index)
				{
					FString LogEntry = FString::Printf(TEXT("%s"), *FileLines[Index]) + LINE_TERMINATOR;
					LogFile->Serialize(TCHAR_TO_ANSI(*LogEntry), LogEntry.Len());
				}

				LogFile->Close();
				delete LogFile;

				// A report has been generated
				ReportGenerated = true;
			}
		}

		return ReportGenerated;
	}

	static bool WriteNetReport(bool IsServer, const FString& FileLine)
	{
		FString FileName;

		if (IsServer)
		{
			FileName = "D:\\Server.txt";
		}
		else
		{
			FileName = "D:\\Client.txt";
		}

		TArray<FString> FileLines;
		FileLines.Add(FileLine);
		return WriteCustomReport(FileName, FileLines);
	}