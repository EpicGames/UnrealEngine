// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ChaosModularVehicle/ChaosSimModuleManagerAsyncCallback.h"
#include "CollisionQueryParams.h"

class UModularVehicleComponent;

struct FModularVehicleDefaultAsyncInput : public FModularVehicleAsyncInput
{
	mutable FCollisionQueryParams TraceParams;
	mutable FCollisionResponseContainer TraceCollisionResponse;

	FModularVehicleDefaultAsyncInput();

	virtual TUniquePtr<FModularVehicleAsyncOutput> Simulate(UWorld* World, const float DeltaSeconds, const float TotalSeconds, bool& bWakeOut) const override;
	virtual void ApplyDeferredForces() const override;

	void SetVehicle(UModularVehicleComponent* VehicleIn) { GCVehicle = VehicleIn; }
	UModularVehicleComponent* GetVehicle() { return GCVehicle; }

	private:

	UModularVehicleComponent* GCVehicle;
};