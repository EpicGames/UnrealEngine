// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosModularVehicle/ChaosSimModuleManagerAsyncCallback.h"
#include "Delegates/Delegate.h"
#include "SimModule/SimModuleTree.h"
#include "SimModule/SimModulesInclude.h"
#include "ModularVehicleBuilder.h"
#include "Chaos/ParticleHandle.h"
#include "Chaos/ParticleHandleFwd.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/TransactionallySafeRWLock.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

DECLARE_LOG_CATEGORY_EXTERN(LogModularVehicleSim, Log, All);

struct FModularVehicleAsyncInput;
struct FChaosSimModuleManagerAsyncOutput;
struct FModuleInputContainer;

struct FModularVehicleDebugParams
{
	bool ShowDebug = false;
	bool SuspensionRaycastsEnabled = true;
	bool ShowSuspensionRaycasts = false;
	bool ShowWheelData = false;
	bool ShowRaycastMaterial = false;
	bool ShowWheelCollisionNormal = false;

	bool DisableAnim = false;
	float FrictionOverride = 1.0f;
};

namespace Chaos
{
	class FClusterUnionPhysicsProxy;
	class FSingleParticlePhysicsProxy;
}

class FModularVehicleSimulation;
DECLARE_MULTICAST_DELEGATE_EightParams(FModularVehicleSimulationCallback, FModularVehicleSimulation*, UWorld*, float, const FModularVehicleAsyncInput&, Chaos::FAllInputs&, IPhysicsProxyBase*, Chaos::FPBDRigidParticleHandle*, Chaos::FSimModuleTree*);

class FModularVehicleSimulation
{
public:
	using FInputNameMap = FInputInterface::FInputNameMap;

	FModularVehicleSimulation(bool InUsingNetworkPhysicsPrediction, int8 InNetMode)
		: bUsingNetworkPhysicsPrediction(InUsingNetworkPhysicsPrediction)
		, NetMode(InNetMode)
	{
	}

	virtual ~FModularVehicleSimulation()
	{
		Terminate();
	}

	UE_API void Initialize(TUniquePtr<Chaos::FSimModuleTree>& InSimModuleTree);
	UE_API void Terminate();

	void SetInputQuantizationType(EModuleInputQuantizationType InInputQuantizationType)
	{ 
		UE::TWriteScopeLock InputConfigLock(InputConfigurationLock);
		InputQuantizationType = InInputQuantizationType; 
	}	

	void SetInputMappings(const FInputNameMap& InNameMap)
	{ 
		UE::TWriteScopeLock InputConfigLock(InputConfigurationLock);
		InputNameMap = InNameMap; 
	}

	void SetStateMappings(const FInputNameMap& InNameMap)
	{
		UE::TWriteScopeLock InputConfigLock(InputConfigurationLock);
		StateNameMap = InNameMap;
	}

	void SetTestInputBuffer(TArray<FModuleInputContainer>& InTestInputBuffer, bool bInIsLoopBuffer, int32 InTestInputBufferStartFrame)
	{
		UE::TWriteScopeLock InputConfigLock(InputConfigurationLock);
		bIsLoopingTestInputBuffer = bInIsLoopBuffer;
		TestInputBuffer = InTestInputBuffer;
		TestInputBufferStartFrame = InTestInputBufferStartFrame;
	}

	UE_API void AppendTreeUpdates(Chaos::FSimTreeUpdates* InNextTreeUpdatesInternal);
	UE_API void ActionTreeUpdates();
	UE_API void GenerateReplicationStructure(FNetworkModularVehicleStates& State);

	UE_API void CacheRootParticle(IPhysicsProxyBase* Proxy);

	/** Update called from Physics Thread */
	UE_API virtual void Simulate(UWorld* InWorld, float DeltaSeconds, const FModularVehicleAsyncInput& InputData, FModularVehicleAsyncOutput& OutputData, IPhysicsProxyBase* Proxy);
	UE_API virtual void SimulateModuleTree(UWorld* InWorld, float DeltaSeconds, const FModularVehicleAsyncInput& InputData, FModularVehicleAsyncOutput& OutputData, IPhysicsProxyBase* Proxy);

	UE_API virtual void OnContactModification(Chaos::FCollisionContactModifier& Modifier, IPhysicsProxyBase* Proxy);

	UE_API void ApplyDeferredForces(IPhysicsProxyBase* Proxy);

	UE_API void PerformAdditionalSimWork(float DeltaSeconds, UWorld* InWorld, const FModularVehicleAsyncInput& InputData, IPhysicsProxyBase* Proxy, Chaos::FAllInputs& AllInputs);

	UE_API void FillOutputState(FModularVehicleAsyncOutput& Output);

	const TUniquePtr<Chaos::FSimModuleTree>& GetSimComponentTree() const {
		Chaos::EnsureIsInPhysicsThreadContext();
		return SimModuleTree;
		}

	TUniquePtr<Chaos::FSimModuleTree>& AccessSimComponentTree() {
		return SimModuleTree; 
		}

	TUniquePtr<Chaos::FSimModuleTree> SimModuleTree;	/* Simulation modules stored in tree structure */
	Chaos::FAllInputs SimInputData;
	bool bUsingNetworkPhysicsPrediction;

	/** Current control inputs that is being used on the PT */
	FModularVehicleInputs VehicleInputs;
	EModuleInputQuantizationType InputQuantizationType = EModuleInputQuantizationType::Default_16Bits;
	FInputNameMap InputNameMap;
	FInputNameMap StateNameMap;
	FTransactionallySafeRWLock InputConfigurationLock;
	FTransactionallySafeRWLock TreeConfigurationLock;

	int8 NetMode;

	int32 TestInputBufferStartFrame = -1;
	bool bIsLoopingTestInputBuffer = false;
	bool ImplementsTestBuffer() { return TestInputBuffer.Num() > 0; }
	bool ImplementsLoopingTestBuffer() { return bIsLoopingTestInputBuffer; }
	TArray<FModuleInputContainer> TestInputBuffer;
	TArray<Chaos::FSimTreeUpdates> NextTreeUpdatesInternal;
	TArray<Chaos::FCreatedModule> NewlyCreatedModuleGuids;

	Chaos::FPBDRigidParticleHandle* RootParticle = nullptr; // cached root particle
	
	FModularVehicleSimulationCallback VehicleSimulationCallback;
};

#undef UE_API
