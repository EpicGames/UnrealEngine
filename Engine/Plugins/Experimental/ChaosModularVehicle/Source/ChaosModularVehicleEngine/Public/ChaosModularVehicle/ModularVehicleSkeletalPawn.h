// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Pawn.h"

#include "ModularVehicleSkeletalPawn.generated.h"

class USkeletalMeshComponent;
class UModularVehicleBaseComponent;

UCLASS(MinimalAPI)
class AModularVehicleSkeletalPawn: public APawn
{
	GENERATED_UCLASS_BODY()

public:
	UFUNCTION()
	USkeletalMeshComponent* GetSkeletalMeshComponent() const { return SkeletalMeshComponent; }

	UFUNCTION()
	UModularVehicleBaseComponent* GetVehicleSimulationComponent() const { return VehicleSimComponent; }

	/* SkeletalMeshComponent root component */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Vehicle, meta = (ExposeFunctionCategories = "Components|ModularVehicle", AllowPrivateAccess = "true"))
	TObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;

	/* ModularVehicleComponent */
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = Vehicle, meta = (ExposeFunctionCategories = "Components|ModularVehicle", AllowPrivateAccess = "true"))
	TObjectPtr<UModularVehicleBaseComponent> VehicleSimComponent;

};
