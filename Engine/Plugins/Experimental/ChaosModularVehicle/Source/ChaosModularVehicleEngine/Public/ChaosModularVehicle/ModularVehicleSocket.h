// Copyright Epic Games, Inc. All Rights Reserved.


#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "UObject/ScriptMacros.h"
#include "ModularVehicleSocket.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

class UClusterUnionVehicleComponent;

USTRUCT(BlueprintType)
struct FModularVehicleSocket
{
	GENERATED_USTRUCT_BODY()

	UE_API FModularVehicleSocket();

	/**
	 *	Defines a named attachment location on the Modular vehicle.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Socket Parameters")
	FName SocketName;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Socket Parameters")
	FVector RelativeLocation;

	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category = "Socket Parameters")
	FRotator RelativeRotation;

	UE_API FVector GetLocation(const class UClusterUnionVehicleComponent* Component) const;

	/** returns FTransform of Socket local transform */
	UE_API FTransform GetLocalTransform() const;

	/** Utility that returns the current transform for this socket. */
	UE_API FTransform GetTransform(const class UClusterUnionVehicleComponent* Component) const;

};



#undef UE_API
