// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Iris/ReplicationSystem/NetTokenStructDefines.h"
#include "Iris/ReplicationSystem/StructNetTokenDataStore.h"
#include "Net/Iris/ReplicationSystem/StructNetTokenDataStoreHelper.h"
#include "SimModule/ModuleInput.h"
#include "ModuleInputTokenStore.generated.h"

USTRUCT()
struct FModuleInputNetTokenData
{
	GENERATED_BODY()
	UPROPERTY()
	TArray<uint8> Types;
	UPROPERTY()
	TArray<bool> DecayValues;
	
	UE_NET_NETTOKEN_GENERATED_BODY(ModuleInputNetTokenData, CHAOSMODULARVEHICLEENGINE_API)
	
	uint64 GetUniqueKey() const
	{
		uint64 HashOfTypes = GetTypeHash(Types);
		uint64 HashOfDecayValues = GetTypeHash(DecayValues);
		return (HashOfTypes<<32) ^ HashOfDecayValues;
	}

	void Init(const TArray<FModuleInputValue>& ModuleInputs)
	{
		Types.Reset(ModuleInputs.Num());
		DecayValues.Reset(ModuleInputs.Num());
		for (int32 Idx = 0; Idx < ModuleInputs.Num(); Idx++)
		{
			Types.Add(static_cast<uint8>(ModuleInputs[Idx].GetValueType()));
			DecayValues.Add(ModuleInputs[Idx].ShouldApplyInputDecay());
		}
	}
};

UE_NET_DECLARE_NAMED_NETTOKEN_STRUCT_SERIALIZERS(ModuleInputNetTokenData, CHAOSMODULARVEHICLEENGINE_API);
