// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectPtr.h"

#include "SimModuleActor.generated.h"

class UVehicleSimBaseComponent;

UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), hidecategories = (PlanarMovement, "Components|Movement|Planar", Activation, "Components|Activation"))
class ASimModuleActor : public AActor
{
	GENERATED_UCLASS_BODY()

public:

	/* Game state callback */
	void TickSimulation(float DeltaTime) {}
	virtual void Tick(float DeltaSeconds) override {}

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModularVehicle)
	TObjectPtr<UStaticMeshComponent> MeshComp;

	// Simulation Component
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModularVehicle)
	TObjectPtr<UVehicleSimBaseComponent> SimComponent;

};

