// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SimModule/SuspensionBaseInterface.h"
#include "Chaos/ChaosEngineInterface.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

class FSuspensionSimModule;

namespace Chaos
{
	struct FAllInputs;
	class FSimModuleTree;
	class FClusterUnionPhysicsProxy;
}

struct FSuspensionSimModuleData 
	: public Chaos::FModuleNetData
	, public Chaos::TSimulationModuleTypeable<FSuspensionSimModule,FSuspensionSimModuleData>
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	FSuspensionSimModuleData(int NodeArrayIndex, const FString& InDebugString) : Chaos::FModuleNetData(NodeArrayIndex, InDebugString) {}
#else
	FSuspensionSimModuleData(int NodeArrayIndex) : FModuleNetData(NodeArrayIndex) {}
#endif

	UE_API virtual void FillSimState(Chaos::ISimulationModuleBase* SimModule) override;

	UE_API virtual void FillNetState(const Chaos::ISimulationModuleBase* SimModule) override;

	virtual void Serialize(FArchive& Ar) override
	{
		Ar << SpringDisplacement;
		Ar << LastDisplacement;
	}

	UE_API virtual void Lerp(const float LerpFactor, const FModuleNetData& Min, const FModuleNetData& Max) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_API virtual FString ToString() const override;
#endif

	float SpringDisplacement = 0.0f;
	float LastDisplacement = 0.0f;
};

struct FSuspensionOutputData 
	: public Chaos::FSimOutputData
	, public Chaos::TSimulationModuleTypeable<FSuspensionSimModule,FSuspensionOutputData>
{
	virtual FSimOutputData* MakeNewData() override { return FSuspensionOutputData::MakeNew(); }
	static FSimOutputData* MakeNew() { return new FSuspensionOutputData(); }

	UE_API virtual void FillOutputState(const Chaos::ISimulationModuleBase* SimModule) override;
	UE_API virtual void Lerp(const FSimOutputData& InCurrent, const Chaos::FSimOutputData& InNext, float Alpha) override;

	UE_API virtual void GetFinalAnimDataGameThread(const FTransform& NewTransform, Chaos::FSimModuleAnimationData& AnimDataOut) override;

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_API virtual FString ToString() override;
#endif

	float SpringDisplacement = 0.0f;
	FVector SpringDisplacementVector = FVector::ZeroVector;
	float SpringSpeed = 0.0f;
	FVector ImpactNormal = FVector(0.0f,0.0f,1.0f);

	FVector ImpactPosition = FVector::ZeroVector;
	FVector ModuleLocalPosition = FVector::ZeroVector;
	FVector LocalSuspensionAxis = FVector::ZeroVector;
	float MaxRaise = 0.0f;
	float MaxDrop = 0.0f;
	bool bWheelHit = false;
};

struct FSuspensionSettings
{
	FSuspensionSettings()
		: SuspensionAxis(FVector(0.f, 0.f, -1.f))
		, RestOffset(FVector::ZeroVector)
		, MaxRaise(5.f)
		, MaxDrop(5.f)
		, MaxLength(0.f)
		, SpringRate(1.f)
		, SpringPreload(0.5f)
		, SpringDamping(0.9f)
		, SuspensionForceEffect(100.0f)
	{

	}

	FVector SuspensionAxis;		// local axis, direction of suspension force raycast traces
	FVector RestOffset;
	float MaxRaise;				// distance [cm]
	float MaxDrop;				// distance [cm]
	float MaxLength;			// distance [cm]

	float SpringRate;			// spring constant
	float SpringPreload;		// Amount of Spring force (independent spring movement)
	float SpringDamping;		// limit compression/rebound speed

	float SuspensionForceEffect; // force that presses the wheels into the ground - producing grip
};


class FSuspensionFactory : public Chaos::IFactoryModule
{
public:
	virtual TSharedPtr<Chaos::FModuleNetData> GenerateNetData(const int32 SimArrayIndex) const override
	{
		return MakeShared<FSuspensionSimModuleData>(
			SimArrayIndex
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			, TEXT("ConstraintSuspension")
#endif			
		);
	}
};

class FSuspensionSimModule 
	: public Chaos::FSuspensionBaseInterface
	, public Chaos::TSimModuleSettings<FSuspensionSettings>
	, public Chaos::TSimulationModuleTypeable<FSuspensionSimModule>
{
	friend FSuspensionSimModuleData;
	friend FSuspensionOutputData;

public:
	DEFINE_CHAOSSIMTYPENAME(FSuspensionSimModule);
	UE_API FSuspensionSimModule(const FSuspensionSettings& Settings);

	virtual TSharedPtr<Chaos::FModuleNetData> GenerateNetData(const int32 SimArrayIndex) const override
	{
		return MakeShared<FSuspensionSimModuleData>(
			SimArrayIndex
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
			, GetDebugName()
#endif			
		);
	}
	UE_API virtual ~FSuspensionSimModule() override;


	virtual Chaos::FSimOutputData* GenerateOutputData() const override
	{
		return FSuspensionOutputData::MakeNew();
	}

	virtual const FString GetDebugName() const { return TEXT("Suspension"); }

	virtual float GetMaxSpringLength() const override { return Setup().MaxLength; }
	UE_API virtual float GetSpringLength() const override;
	UE_API virtual  void SetSpringLength(float InLength, float WheelRadius) override;
	UE_API virtual void GetWorldTraceEndpoints(float DeltaSeconds, const FTransform& BodyTransform, const FVector& Velocity, float WheelRadius, Chaos::FSpringTrace& OutTrace) const override;
	UE_API virtual void OnConstruction_External(const Chaos::FPhysicsObjectHandle& PhysicsObject) override;
	UE_API virtual void OnTermination_External() override;

	UE_API virtual void Simulate(float DeltaTime, const Chaos::FAllInputs& Inputs, Chaos::FSimModuleTree& VehicleModuleSystem) override;

	UE_API virtual void Animate() override;

	const FVector& GetRestLocation() const { return Setup().RestOffset; }

	UE_API void UpdateConstraint();

protected:
	UE_API void CreateConstraint(const Chaos::FPhysicsObjectHandle& PhysicsObject);
	UE_API void DestroyConstraint();

private:

	float SpringDisplacement = 0.0f;
	float LastDisplacement = 0.0f;
	float SpringSpeed = 0.0f;
	float CurrentTimeDilation = 1.0f;
	FPhysicsConstraintHandle ConstraintHandle;
};


class FSuspensionSimFactory
		: public Chaos::FSimFactoryModule<FSuspensionSimModuleData>
		, public Chaos::TSimulationModuleTypeable<FSuspensionSimModule, FSuspensionSimFactory>
		, public Chaos::TSimFactoryAutoRegister<FSuspensionSimFactory>
	
{
public:
	FSuspensionSimFactory() : FSimFactoryModule(TEXT("SuspensionSimFactory")) {}
};

#undef UE_API
