// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "VehicleSimBaseComponent.h"
#include "VehicleSimAerofoilComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API



UENUM()
enum class EModuleAerofoilType : uint8
{
	Fixed = 0,
	Wing,			// affected by Roll input
	Rudder,			// affected by steering/yaw input
	Elevator		// affected by Pitch input
};

UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), meta = (BlueprintSpawnableComponent), hidecategories = (Object, Tick, Replication, Cooking, Activation, LOD))
class UVehicleSimAerofoilComponent : public UVehicleSimBaseComponent
{
	GENERATED_BODY()

public:
	UE_API UVehicleSimAerofoilComponent();
	virtual ~UVehicleSimAerofoilComponent() = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FVector Offset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FVector ForceAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FVector ControlRotationAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float Area;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float Camber;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float MaxControlAngle;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float StallAngle;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	EModuleAerofoilType Type;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float LiftMultiplier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float DragMultiplier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float AnimationMagnitudeMultiplier;

	virtual ESimModuleType GetModuleType() const override { return ESimModuleType::Aerofoil; }

	UE_API virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const override;

};

#undef UE_API
