// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SceneComponent.h"
#include "SimModule/ModuleInput.h"

#include "VehicleSimBaseComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

namespace Chaos
{
	class ISimulationModuleBase;
	struct FSimOutputData;
}

UENUM()
enum class ESimModuleType : uint8
{
	Undefined = 0,
	Chassis,		// no simulation effect
	Thruster,		// applies force
	Aerofoil,		// applied drag and lift forces
	Wheel,			// a wheel will simply roll if it has no power source
	Suspension,		// associated with a wheel
	Axle,			// connects more than one wheel
	Transmission,	// gears - torque multiplier
	Engine,			// (torque curve required) power source generates torque for wheel, axle, transmission, clutch
	Motor,			// (electric?, no torque curve required?) power source generates torque for wheel, axle, transmission, clutch
	Clutch,			// limits the amount of torque transferred between source and destination allowing for different rotation speeds of connected axles
	Wing,			// lift and controls aircraft roll
	Rudder,			// controls aircraft yaw
	Elevator,		// controls aircraft pitch
	Propeller,		// generates thrust when connected to a motor/engine
	Balloon			// TODO: rename anti gravity??
};

/** Interface used for shared functionality between types of base components. */
UINTERFACE(MinimalAPI, BlueprintType, meta = (CannotImplementInterfaceInBlueprint))
class UVehicleSimBaseComponentInterface : public UInterface
{
	GENERATED_BODY()
};

class IVehicleSimBaseComponentInterface
{
	GENERATED_BODY()

public:

	virtual ESimModuleType GetModuleType() const { return ESimModuleType::Undefined; }
	/** Caller takes ownership of pointer to new Sim Module. */
	virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const { return nullptr; }
	virtual FName GetBoneName() const { return NAME_None; }
	virtual const FVector& GetAnimationOffset() const { return FVector::ZeroVector; }
	virtual void SetAnimationEnabled(bool AnimationEnabledIn) { }
	virtual bool GetAnimationEnabled() const { return false; }
	virtual TArray<FModuleInputSetup> GetInputConfig() const { return TArray<FModuleInputSetup>(); }
	virtual int32 GetAnimationSetupIndex() const { return INDEX_NONE; }
	virtual void SetTreeIndex(const int32 NewValue) {}
	virtual int32 GetTreeIndex() const { return INDEX_NONE; }
	virtual void OnAdded() {}
	virtual void OnRemoved() {}
	virtual void OnOutputReady(const Chaos::FSimOutputData* OutputData) {}
};

/** This if for sim components that need scene component properties along with rendering and collision. */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UVehicleSimBaseComponent 
	: public UPrimitiveComponent
	, public IVehicleSimBaseComponentInterface
{
	GENERATED_BODY()

protected:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModularVehicle)
	FName BoneName = NAME_None;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModularVehicle)
	FVector AnimationOffset = FVector::ZeroVector;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = ModularVehicle)
	bool bAnimationEnabled = false;

	UPROPERTY(EditAnywhere, Category = VehicleInput)
	TArray<FModuleInputSetup> InputConfig;

	int32 AnimationSetupIndex = INDEX_NONE;
	int32 TreeIndex = INDEX_NONE; // helper - since Component->GetAttachChildren doesn't contain any data

public:

	/** IVehicleSimBaseComponentInterface overrides */
	virtual FName GetBoneName() const override { return BoneName; }
	virtual const FVector& GetAnimationOffset() const override { return AnimationOffset; }
	virtual void SetAnimationEnabled(bool AnimationEnabledIn) override { bAnimationEnabled = AnimationEnabledIn; }
	virtual bool GetAnimationEnabled() const override { return bAnimationEnabled; }
	virtual TArray<FModuleInputSetup> GetInputConfig() const override { return InputConfig; }
	virtual int32 GetAnimationSetupIndex() const override { return AnimationSetupIndex; }
	UE_API virtual void SetTreeIndex(const int32 NewValue) override;
	virtual int32 GetTreeIndex() const override { return TreeIndex; }
	/** END IVehicleSimBaseComponentInterface overrides */
};

/** This if for sim components that need transform and attachment, no rendering, no collision. */
UCLASS(MinimalAPI, BlueprintType, Blueprintable)
class UVehicleSimBaseSceneComponent 
	: public USceneComponent
	, public IVehicleSimBaseComponentInterface
{
	GENERATED_BODY()

protected:

	UPROPERTY(EditAnywhere, Category = VehicleInput)
	TArray<FModuleInputSetup> InputConfig;

public:

	/** IVehicleSimBaseComponentInterface overrides */
	virtual TArray<FModuleInputSetup> GetInputConfig() const override { return InputConfig; }
	/** END IVehicleSimBaseComponentInterface overrides */
};

#undef UE_API
