// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "VehicleSimBaseComponent.h"
#include "VehicleSimChassisComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API


UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), meta = (BlueprintSpawnableComponent), hidecategories = (Object, Tick, Replication, Cooking, Activation, LOD))
class UVehicleSimChassisComponent : public UVehicleSimBaseComponent
{
	GENERATED_BODY()

public:
	UE_API UVehicleSimChassisComponent();
	virtual ~UVehicleSimChassisComponent() = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float AreaMetresSquared;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float DragCoefficient;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float DensityOfMedium;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float XAxisMultiplier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float YAxisMultiplier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float AngularDamping;

	virtual ESimModuleType GetModuleType() const override { return ESimModuleType::Chassis; }

	UE_API virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const override;
};

#undef UE_API
