// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "VehicleSimBaseComponent.h"
#include "VehicleSimClutchComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API


UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), meta = (BlueprintSpawnableComponent), hidecategories = (Object, Tick, Replication, Cooking, Activation, LOD))
class UVehicleSimClutchComponent : public UVehicleSimBaseComponent
{
	GENERATED_BODY()

public:
	UE_API UVehicleSimClutchComponent();
	virtual ~UVehicleSimClutchComponent() = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
		float ClutchStrength;

	virtual ESimModuleType GetModuleType() const override { return ESimModuleType::Clutch; }

	UE_API virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const override;

};

#undef UE_API
