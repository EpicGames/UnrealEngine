// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "Curves/CurveFloat.h"
#include "VehicleSimBaseComponent.h"
#include "VehicleSimEngineComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API


UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), meta = (BlueprintSpawnableComponent), hidecategories = (Object, Tick, Replication, Cooking, Activation, LOD))
class UVehicleSimEngineComponent : public UVehicleSimBaseComponent
{
	GENERATED_BODY()

public:
	UE_API UVehicleSimEngineComponent();
	virtual ~UVehicleSimEngineComponent() = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FRuntimeFloatCurve TorqueCurve;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float MaxTorque;			// [N.m] The peak torque Y value in the normalized torque graph

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	int MaxRPM;				// [RPM] The absolute maximum RPM the engine can theoretically reach (last X value in the normalized torque graph)

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	int EngineIdleRPM; 		// [RPM] The RPM at which the throttle sits when the car is not moving			

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float EngineBrakeEffect;	// [0..1] How much the engine slows the vehicle when the throttle is released

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float EngineInertia;		// [Kg.m-2] How hard it is to turn the engine

	virtual ESimModuleType GetModuleType() const override { return ESimModuleType::Engine; }
	virtual void OnOutputReady(const Chaos::FSimOutputData* OutputData) override;

	UE_API virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const override;

};

#undef UE_API
