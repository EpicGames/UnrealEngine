// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "VehicleSimBaseComponent.h"
#include "VehicleSimSuspensionComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API


UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), meta = (BlueprintSpawnableComponent), hidecategories = (Object, Replication, Cooking, Activation, LOD, Physics, Collision, AssetUserData, Event))
class UVehicleSimSuspensionComponent : public UVehicleSimBaseComponent
{
	GENERATED_BODY()

public:

	UE_API UVehicleSimSuspensionComponent();
	virtual ~UVehicleSimSuspensionComponent() = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FVector SuspensionAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float SuspensionMaxRaise;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float SuspensionMaxDrop;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float SpringRate;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float SpringPreload;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float SpringDamping;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float SuspensionForceEffect;

	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	//float SwaybarEffect;

	virtual ESimModuleType GetModuleType() const override { return ESimModuleType::Suspension; }
	UE_API virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const override;
};

#undef UE_API
