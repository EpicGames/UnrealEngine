// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "VehicleSimBaseComponent.h"
#include "VehicleSimThrusterComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API


UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), meta = (BlueprintSpawnableComponent), hidecategories = (Object, Tick, Replication, Cooking, Activation, LOD))
class UVehicleSimThrusterComponent : public UVehicleSimBaseComponent
{
	GENERATED_BODY()

public:

	UE_API UVehicleSimThrusterComponent();
	virtual ~UVehicleSimThrusterComponent() = default;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float MaxThrustForce;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FVector ForceAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FVector ForceOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	bool bSteeringEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FVector SteeringAxis;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float MaxSteeringAngle;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float SteeringForceEffect;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float BoostMultiplierEffect;

	virtual ESimModuleType GetModuleType() const override { return ESimModuleType::Thruster; }

	UE_API virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const override;
};

#undef UE_API
