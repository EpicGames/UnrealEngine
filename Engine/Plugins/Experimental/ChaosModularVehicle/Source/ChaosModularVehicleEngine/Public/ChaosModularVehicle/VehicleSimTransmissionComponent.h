// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "VehicleSimBaseComponent.h"
#include "VehicleSimTransmissionComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API


UENUM()
enum class EModuleTransType : uint8
{
	Manual = 0,
	Automatic
};


UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), meta = (BlueprintSpawnableComponent), hidecategories = (Object, Tick, Replication, Cooking, Activation, LOD))
class UVehicleSimTransmissionComponent : public UVehicleSimBaseComponent
{
	GENERATED_BODY()

public:
	UE_API UVehicleSimTransmissionComponent();
	virtual ~UVehicleSimTransmissionComponent() = default;

	// native (fast, low overhead) versions 
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnGearChangeNative, int, int);
	FOnGearChangeNative OnGearChangeNativeEvent;

	// standard events
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnGearChange, int, Guid, int, CurrentGear);
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnGearChange OnGearChangeEvent;


	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	TArray<float> ForwardRatios;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	TArray<float> ReverseRatios;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float FinalDriveRatio;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	int ChangeUpRPM;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	int ChangeDownRPM;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float GearChangeTime;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float GearHysteresisTime;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float TransmissionEfficiency;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	EModuleTransType TransmissionType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	bool AutoReverse;

	virtual ESimModuleType GetModuleType() const override { return ESimModuleType::Transmission; }
	virtual void OnOutputReady(const Chaos::FSimOutputData* OutputData) override;

	UE_API virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const override;

};

#undef UE_API
