// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectPtr.h"
#include "VehicleSimBaseComponent.h"
#include "VehicleSimWheelComponent.generated.h"

#define UE_API CHAOSMODULARVEHICLEENGINE_API

UENUM()
enum class EWheelAxisType : uint8
{
	X = 0,		// X forwards
	Y			// Y forwards
};


UCLASS(MinimalAPI, ClassGroup = (ModularVehicle), meta = (BlueprintSpawnableComponent), hidecategories = (Object, Replication, Cooking, Activation, LOD, Physics, Collision, AssetUserData, Event))
class UVehicleSimWheelComponent : public UVehicleSimBaseComponent
{
	GENERATED_BODY()

public:

	UE_API UVehicleSimWheelComponent();
	virtual ~UVehicleSimWheelComponent() = default;

	// native (fast, low overhead) versions 
	DECLARE_MULTICAST_DELEGATE_TwoParams(FOnWheelTouchChangeNative, int, bool);
	FOnWheelTouchChangeNative OnWheelTouchChangeNativeEvent;

	// standard events
	DECLARE_DYNAMIC_MULTICAST_DELEGATE_TwoParams(FOnWheelTouchChange, int, Guid, bool, IsInContact);
	UPROPERTY(BlueprintAssignable, Category = "Events")
	FOnWheelTouchChange OnWheelTouchChangeEvent;

	// - Wheel --------------------------------------------------------
	//UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	//float WheelMass;			// Mass of wheel [Kg]

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float WheelRadius;			// [cm]

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float WheelWidth;			// [cm]

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float WheelInertia;

	// grip and turning related
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float FrictionMultiplier;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float CorneringStiffness;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float SlipAngleLimit;

	// #TODO: 
	// LateralSlipGraphMultiplier, LateralSlipGraph

	// - Braking ------------------------------------------------------
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	float MaxBrakeTorque;

	// Handbrake
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	bool bHandbrakeEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes, meta = (EditCondition = "bHandbrakeEnabled"))
	float HandbrakeTorque;

	// - Steering -----------------------------------------------------
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	bool bSteeringEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes, meta = (EditCondition = "bSteeringEnabled"))
	float MaxSteeringAngle;

	// - Other --------------------------------------------------------

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	bool bABSEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	bool bTractionControlEnabled;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	EWheelAxisType AxisType;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	FVector ForceOffset;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Attributes)
	bool ReverseDirection;

	virtual ESimModuleType GetModuleType() const override { return ESimModuleType::Wheel; }
	virtual void OnOutputReady(const Chaos::FSimOutputData* OutputData) override;

	UE_API virtual Chaos::ISimulationModuleBase* CreateNewCoreModule() const override;
};

#undef UE_API
